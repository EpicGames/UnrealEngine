{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Datasmith FBX Importer",
	"Description" : "Adds support for importing content from DeltaGen and VRED into Unreal Engine",
	"Category" : "Importers",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "DatasmithVREDTranslator",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name": "DatasmithDeltaGenTranslator",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "DatasmithFBXTranslator",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins":
	[
		{
			"Name": "DatasmithImporter",
			"Enabled": true
		},
        {
			"Name": "DatasmithContent",
			"Enabled": true
		}
    ]
}
