// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenAnimationInterpolator.h"

#include "Curves/CurveEvaluation.h"
#include "Math/Vector.h"

namespace DeltaGen
{
	/**
	 * Finds the indices of the values from InTimes that are closest to InTime, and returns an interpolation constant for them
	 * OutIndex1: First index of InTimes that has a value <= InTime
	 * OutIndex2: First index of InTimes that has a value > InTime
	 * OutLerpFactor: Value in [0, 1] describing the position of InTime between the values with indices OutIndex1, OutIndex2
	 */
	void InterpolateTime(const TArray<float>& InTimes, float InTime, int32& OutIndex1, int32& OutIndex2, float& OutLerpFactor)
	{
		const int32 NumTimes = InTimes.Num();

		int32 Index2 = Algo::UpperBound(InTimes, InTime);
		Index2 = FMath::Min(Index2, NumTimes - 1);

		int32 Index1 = Index2 - 1;
		Index1 = FMath::Max(Index1, 0);

		float Time1 = InTimes[Index1];
		float Time2 = InTimes[Index2];

		OutLerpFactor = FMath::IsNearlyEqual(Time1, Time2) ? 0.0f : (InTime - Time1) / (Time2 - Time1);
		OutIndex1 = Index1;
		OutIndex2 = Index2;
	}

	FInterpolator::FInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InValues)
		: Times(InTimes)
		, Values(InValues)
		, bIsValid(false)
	{
	}

	FInterpolator::~FInterpolator()
	{
	}

	float FInterpolator::GetMinTime() const
	{
		if (Times.Num() == 0)
		{
			return 0.0f;
		}

		return Times[0];
	}

	float FInterpolator::GetMaxTime() const
	{
		if (Times.Num() == 0)
		{
			return 0.0f;
		}

		return Times.Last();
	}

	bool FInterpolator::IsValid() const
	{
		return bIsValid;
	}

	FConstInterpolator::FConstInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InValues)
		: FInterpolator(InTimes, InValues)
	{
		bIsValid = (Times.Num() > 0 &&
					Times.Num() == Values.Num());
	}

	FVector FConstInterpolator::Evaluate(float Time) const
	{
		if (!IsValid())
		{
			return FVector();
		}

		int32 Index1 = INDEX_NONE;
		int32 Index2 = INDEX_NONE;
		float LerpFactor = FLT_MAX;
		InterpolateTime(Times, Time, Index1, Index2, LerpFactor);

		return Values[Index1];
	}

	FVector FConstInterpolator::SolveForX(float X) const
	{
		if (!IsValid())
		{
			return FVector::ZeroVector;
		}

		int32 Index = Algo::UpperBoundBy(Values, X, [](const FVector& Value)
		{
			return Value.X;
		});

		return Values[Index > 0 ? Index - 1 : 0];
	}

	FLinearInterpolator::FLinearInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InValues)
		: FInterpolator(InTimes, InValues)
	{
		bIsValid = (Times.Num() > 0 &&
					Times.Num() == Values.Num());
	}

	FVector FLinearInterpolator::Evaluate(float Time) const
	{
		if (!IsValid())
		{
			return FVector::ZeroVector;
		}

		int32 Index1 = INDEX_NONE;
		int32 Index2 = INDEX_NONE;
		float LerpFactor = FLT_MAX;
		InterpolateTime(Times, Time, Index1, Index2, LerpFactor);

		return FMath::Lerp(Values[Index1], Values[Index2], LerpFactor);
	}

	FVector FLinearInterpolator::SolveForX(float X) const
	{
		if (!IsValid())
		{
			return FVector::ZeroVector;
		}

		int32 Index2 = Algo::UpperBoundBy(Values, X, [](const FVector& Value)
		{
			return Value.X;
		});
		Index2 = FMath::Min(Index2, Values.Num() - 1);
		int32 Index1 = FMath::Max(Index2 - 1, 0);

		const FVector& Value1 = Values[Index1];
		const FVector& Value2 = Values[Index2];

		float LerpFactor = (X - Value1.X) / (Value2.X - Value1.X);

		return FMath::Lerp(Value1, Value2, LerpFactor);
	}

	FCubicInterpolator::FCubicInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InControlPoints)
		: FInterpolator(InTimes, InControlPoints)
	{
		// We should have 1 tangential (actual vertex) and 2 auxiliary (handles) control points per key, except the
		// first and last keys, which have 1 handle less, so NumControlPts = NumKeys + NumKeys * 2 - 2
		bIsValid = (Values.Num() > 0 &&
					Times.Num() > 0 &&
					Values.Num() == Times.Num() * 3 - 2);
	}

	FVector FCubicInterpolator::Evaluate(float Time) const
	{
		if (!IsValid())
		{
			return FVector::ZeroVector;
		}

		int32 Index1 = INDEX_NONE;
		int32 Index2 = INDEX_NONE;
		float LerpFactor = FLT_MAX;
		InterpolateTime(Times, Time, Index1, Index2, LerpFactor);

		const FVector& LeaveControlPoint  = Index1 < Times.Num() - 1 ? Values[3 * Index1 + 1] : Values[3 * Index1];
		const FVector& ArriveControlPoint = Index2 > 0 ?               Values[3 * Index2 - 1] : Values[3 * Index2];

		// Index1 * 3 and Index2 * 3 because Values stores arrive/leave control points too
		return UE::Curves::BezierInterp(Values[Index1 * 3],
							LeaveControlPoint,
							ArriveControlPoint,
							Values[Index2 * 3],
							LerpFactor);
	}

	FVector FCubicInterpolator::SolveForX(float X) const
	{
		if (!IsValid())
		{
			return FVector::ZeroVector;
		}

		int32 NumValues = Values.Num();

		// Earlying-out when outside the range is important, as we won't find usable
		// solutions there. We could potentially early out when we are sufficiently close
		// to any tangent control point, but I'm not sure whether that would be an optimization
		// or not for the general case
		if (X >= Values[NumValues - 1].X)
		{
			return Values[NumValues - 1];
		}
		else if (X <= Values[0].X)
		{
			return Values[0];
		}

		// Find the target N=4 Bezier curve segment
		int32 Index2 = INDEX_NONE;
		for (Index2 = 0; Index2 < Values.Num(); Index2 += 3) // Skip arrive/leave control points
		{
			const FVector& Value = Values[Index2];
			if (Value.X > X)
			{
				break;
			}
		}
		Index2 = FMath::Min(Index2, Values.Num() - 1);
		int32 Index1 = FMath::Max(Index2 - 3, 0);

		// Find Bezier curve segment control points
		const FVector& P0 = Values[Index1];
		const FVector& P1 = Values[FMath::Min(Index1 + 1, Values.Num() - 1)];
		const FVector& P2 = Values[FMath::Max(Index2 - 1, 0)];
		const FVector& P3 = Values[Index2];

		// Early out if we can to avoid SolveCubic. Also because if our X is exactly the same
		// as one of the points we might run into some precision problems within SolveCubic
		// e.g. the valid solution is -1.9E-20 and is discarded because its less than zero
		if (FMath::IsNearlyEqual(P0.X, (FVector::FReal)X))
		{
			return P0;
		}
		else if (FMath::IsNearlyEqual(P3.X, (FVector::FReal)X))
		{
			return P3;
		}

		// Find coeffients for the Bezier curve cubic polynomial in power form
		const double CoefT3 =     - P0.X + 3 * P1.X - 3 * P2.X + P3.X;
		const double CoefT2 =   3 * P0.X - 6 * P1.X + 3 * P2.X;
		const double CoefT1 = - 3 * P0.X + 3 * P1.X;
		const double CoefConst =    P0.X - X;

		// Solve cubic polynomial with Gerolano Cardano's formula
		double Coefs[4] = { CoefConst, CoefT1, CoefT2, CoefT3 };
		double Solutions[3] = {-1., -1., -1.};
		int32 NumSolutions = UE::Curves::SolveCubic(Coefs, Solutions);

		// The target solution, if it exists, is the only real one within [0, 1]
		float TargetAlpha = FLT_MAX;
		for (const double Solution : Solutions)
		{
			if (Solution < 0.0 || Solution > 1.0)
			{
				continue;
			}
			TargetAlpha = Solution;
			break;
		}
		if (TargetAlpha == FLT_MAX)
		{
			return FVector::ZeroVector;
		}

		// TargetAlpha is our bezier interpolation constant, but its in [0, 1] with
		// respect to the current bezier segment. Here we map it to [MinTime, MaxTime],
		// so that it can be used as a global curve interpolation constant for Evaluate
		float Time = FMath::Lerp(Times[Index1 / 3], Times[Index2 / 3], TargetAlpha);

		return Evaluate(Time);
	}
}
