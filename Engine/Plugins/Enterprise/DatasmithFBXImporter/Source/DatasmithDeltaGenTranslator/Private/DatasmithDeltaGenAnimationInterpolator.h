// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

namespace DeltaGen
{
	/**
	 * Utility to interpolate animation curves with different types of interpolations.
	 * This is used instead of other existing utilities as this allows interpolation directly
	 * with Bezier control points, without going through tangents and tangent weights
	 */
	class FInterpolator
	{
	public:
		FInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InValues);
		virtual ~FInterpolator();

		/** Evaluates the interpolator for when the interpolation constant is Time */
		virtual FVector Evaluate(float Time) const = 0;

		/**
		 * Finds an interpolated value that has a certain X.
		 * Requires that Values contains points that are function of X (so no more than
		 * one Value for each given x). DeltaGen enforces this constraint for timing curves
		 */
		virtual FVector SolveForX(float X) const = 0;

		/** Gets the smallest stored Time value */
		float GetMinTime() const;

		/** Gets the largest stored Time value */
		float GetMaxTime() const;

		/** Returns whether our arrays are valid for interpolation */
		bool IsValid() const;

	protected:
		const TArray<float>& Times;
		const TArray<FVector>& Values;
		bool bIsValid;
	};

	class FConstInterpolator : public FInterpolator
	{
	public:
		FConstInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InValues);
		virtual FVector Evaluate(float Time) const override;
		virtual FVector SolveForX(float X) const override;
	};

	class FLinearInterpolator : public FInterpolator
	{
	public:
		FLinearInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InValues);
		virtual FVector Evaluate(float Time) const override;
		virtual FVector SolveForX(float X) const override;
	};

	/**
	 * Corresponds to DeltaGen's "Smooth" type of interpolation curve, which is the same as
	 * cubic interpolation, or concatenated N=4 Bezier curves (e.g. P0, P1, P2 and P3 for each segment)
	 */
	class FCubicInterpolator : public FInterpolator
	{
	public:
		FCubicInterpolator(const TArray<float>& InTimes, const TArray<FVector>& InValues);
		virtual FVector Evaluate(float Time) const override;
		virtual FVector SolveForX(float X) const override;
	};
}
