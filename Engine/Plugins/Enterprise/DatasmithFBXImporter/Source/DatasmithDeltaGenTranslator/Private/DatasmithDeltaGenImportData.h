// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DataTable.h"

enum class EObjectSetDataType : uint8
{
	None,
	Translation,
	Rotation,
	Scaling,
	Visibility,
	Center,
};

enum class EDeltaGenVarDataVariantSwitchType : uint8
{
	Unsupported,
	Camera,
	Geometry,
	Package,
	SwitchObject,
	ObjectSet
};

struct FDeltaGenVarDataGeometryVariant
{
	FString Name;
	TArray<FName> VisibleMeshes;
	TArray<FName> HiddenMeshes;
};

struct FDeltaGenVarDataCameraVariant
{
	FString Name;
	FVector Location;
	FRotator Rotation;
};

struct FDeltaGenVarDataPackageVariant
{
	FString Name;
	TArray<int32> SelectedVariants; // variant id for variant selected in  each variant set(TargetVariantSet)
};

struct FDeltaGenVarDataSwitchObjectVariant
{
	FString Name;
	int32 Selection;
};

struct FDeltaGenVarDataObjectSetVariantValue
{
	FName TargetNodeNameSanitized = NAME_None;
	EObjectSetDataType DataType = EObjectSetDataType::None;
	TArray<uint8> Data;
};

struct FDeltaGenVarDataObjectSetVariant
{
	FString Name;
	TArray<FDeltaGenVarDataObjectSetVariantValue> Values;
};

struct FDeltaGenVarDataVariantSwitchCamera
{
	TArray<FDeltaGenVarDataCameraVariant> Variants;
};

struct FDeltaGenVarDataVariantSwitchGeometry
{
	TArray<FName> TargetNodes;
	TArray<FDeltaGenVarDataGeometryVariant> Variants;
};

struct FDeltaGenVarDataVariantSwitchPackage
{
	TArray<FString> TargetVariantSets;
	TArray<FDeltaGenVarDataPackageVariant> Variants;
};

struct FDeltaGenVarDataVariantSwitchSwitchObject
{
	FName TargetSwitchObject;
	TArray<FDeltaGenVarDataSwitchObjectVariant> Variants;
};

struct FDeltaGenVarDataVariantSwitchObjectSet
{
	TArray<FDeltaGenVarDataObjectSetVariant> Variants;
};

struct FDeltaGenVarDataVariantSwitch : public FTableRowBase
{
	FString Name;
	EDeltaGenVarDataVariantSwitchType Type;
	TMap<int32, int32> VariantIDToVariantIndex;
	TMap<int32, FString> VariantIDToVariantName;

	FDeltaGenVarDataVariantSwitchCamera Camera;
	FDeltaGenVarDataVariantSwitchGeometry Geometry;
	FDeltaGenVarDataVariantSwitchSwitchObject SwitchObject;
	FDeltaGenVarDataVariantSwitchPackage Package;
	FDeltaGenVarDataVariantSwitchObjectSet ObjectSet;
};

struct FDeltaGenVarData
{
	TArray<FDeltaGenVarDataVariantSwitch> VariantSwitches;
};

struct FDeltaGenPosDataState : public FTableRowBase
{
	// Name of the actual state
	FString Name;

	// Maps Actor name to whether it's on or off (visibility)
	TMap<FString, bool> States;

	// Maps Actor name to a switch choice (index of the child that is visible)
	TMap<FName, int> Switches;

	// Maps Actor name to a material name
	TMap<FString, FString> Materials;
};

struct FDeltaGenPosData
{
	TArray<FDeltaGenPosDataState> States;
};

enum class EDeltaGenTmlDataAnimationTrackType : uint8
{
	Unsupported = 0,
	Translation = 1,
	Rotation = 2,
	RotationDeltaGenEuler = 4,
	Scale = 8,
	Center = 16
};
ENUM_CLASS_FLAGS(EDeltaGenTmlDataAnimationTrackType);

enum class EDeltaGenAnimationInterpolation : uint8
{
	Unsupported = 0,
	Constant = 1,
	Linear = 2,
	Cubic = 3,
};

struct FDeltaGenTmlDataAnimationTrack
{
	EDeltaGenTmlDataAnimationTrackType Type = EDeltaGenTmlDataAnimationTrackType::Unsupported;
	EDeltaGenAnimationInterpolation ValueInterpolation = EDeltaGenAnimationInterpolation::Unsupported;
	EDeltaGenAnimationInterpolation KeyInterpolation = EDeltaGenAnimationInterpolation::Unsupported;
	TArray<float> Keys;
	TArray<FVector> KeyControlPoints;
	TArray<FVector> Values;
	TArray<FVector> ValueControlPoints;
	float DelayMs = 0.0f;
};

struct FDeltaGenTmlDataTimelineAnimation
{
	FName TargetNode;
	TArray<FDeltaGenTmlDataAnimationTrack> Tracks;
	float DelayMs = 0.0f;
};

struct FDeltaGenTmlDataTimeline : public FTableRowBase
{
	FString Name;
	float Framerate;
	TArray<FDeltaGenTmlDataTimelineAnimation> Animations;
};

struct FDeltaGenAnimationsData
{
	TArray<FDeltaGenTmlDataTimeline> Timelines;
};
