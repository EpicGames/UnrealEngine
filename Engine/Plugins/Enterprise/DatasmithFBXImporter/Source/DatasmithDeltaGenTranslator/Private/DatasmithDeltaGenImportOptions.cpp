// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenImportOptions.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"


#define LOCTEXT_NAMESPACE "DatasmithDeltaGenImporter"

UDatasmithDeltaGenImportOptions::UDatasmithDeltaGenImportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bRemoveInvisibleNodes(true)
	, bSimplifyNodeHierarchy(true)
	, bImportVar(true)
	, bImportPos(true)
	, bImportTml(true)
	, ShadowTextureMode(EShadowTextureMode::Ignore)
{
}

namespace DeltaGenImportOptionsImpl
{
	FString TryGetPath(const FString& FBXFilePathNoExt, const FString& Extension)
	{
		FString PathAttempt = FPaths::SetExtension(FBXFilePathNoExt, Extension);
		if (FPaths::FileExists(PathAttempt))
		{
			return PathAttempt;
		}

		FString Folder = FPaths::GetPath(FBXFilePathNoExt);

		TArray<FString> CandidateFiles;
		IFileManager::Get().FindFiles(CandidateFiles, *Folder, *Extension);
		if (CandidateFiles.Num() > 0)
		{
			return Folder / CandidateFiles[0];
		}

		return FString();
	}

	FString FindBestFile(const FString& FBXFileWithoutExt, const FString& Extension)
	{
		const FString FBXDirectory = FPaths::GetPath(FBXFileWithoutExt);

		FString VarPathStr = FPaths::SetExtension(FBXFileWithoutExt, Extension);
		if (FPaths::FileExists(VarPathStr))
		{
			return VarPathStr;
		}

		return FString();
	}

	// This will scan for a *.rtll.texturePath file and scan it for lines that contain existing
	// folders. It will also check the usual FBXDirectory\textures and FBXDirectory\shadowtextures
	TArray<FString> FindTexturesFolders(const FString& FBXFileWithoutExt)
	{
		const FString FBXDirectory = FPaths::GetPath(FBXFileWithoutExt);

		// Search for all texturePath files in FBXDirectory
		TArray<FString> TexPathFiles;
		IFileManager::Get().FindFiles(TexPathFiles, *FBXDirectory, TEXT("texturePath"));
		FString IdealFile = FBXFileWithoutExt + TEXT(".rtll.texturePath");
		if (FPaths::FileExists(IdealFile))
		{
			TexPathFiles.Insert(IdealFile, 0);
		}

		// Use texturePath files to fetch for existing texture folders
		TSet<FString> TextureFolders;
		for (const FString& File : TexPathFiles)
		{
			TArray<FString> FileContentLines;
			if (!FFileHelper::LoadFileToStringArray(FileContentLines, *(FBXDirectory / File)))
			{
				continue;
			}

			for (FString Folder : FileContentLines)
			{
				FPaths::NormalizeDirectoryName(Folder);
				Folder = FBXDirectory / Folder;
				FPaths::CollapseRelativeDirectories(Folder);

				// Remove the "Folder/./OtherFolder" and "Folder/." that we normally get with
				// texturePath folders
				Folder += TEXT("/");
				Folder.ReplaceInline(TEXT("/./"), TEXT("/"), ESearchCase::CaseSensitive);
				Folder.RemoveFromEnd(TEXT("/"));

				if (FPaths::DirectoryExists(Folder))
				{
					TextureFolders.Add(Folder);
				}
			}
		}

		FString UsualTexturePath = FPaths::Combine(FBXDirectory, TEXT("textures"));
		if (FPaths::DirectoryExists(UsualTexturePath))
		{
			TextureFolders.Add(UsualTexturePath);
		}

		FString UsualShadowTexturePath = FPaths::Combine(FBXDirectory, TEXT("shadowtextures"));
		if (FPaths::DirectoryExists(UsualShadowTexturePath))
		{
			TextureFolders.Add(UsualShadowTexturePath);
		}

		TArray<FString> Result = TextureFolders.Array();
		Result.Sort();
		return Result;
	}
}

void UDatasmithDeltaGenImportOptions::ResetPaths(const FString& InFBXFilename, bool bJustEmptyPaths)
{
	// Handle file.fbx and file.fbx.intermediate
	FString PathNoExt = FPaths::ChangeExtension(FPaths::ChangeExtension(InFBXFilename, ""), "");

	if (VarPath.FilePath.IsEmpty() || !bJustEmptyPaths)
	{
		VarPath.FilePath = DeltaGenImportOptionsImpl::FindBestFile(PathNoExt, TEXT("var"));
	}
	if (PosPath.FilePath.IsEmpty() || !bJustEmptyPaths)
	{
		PosPath.FilePath = DeltaGenImportOptionsImpl::FindBestFile(PathNoExt, TEXT("pos"));
	}
	if (TmlPath.FilePath.IsEmpty() || !bJustEmptyPaths)
	{
		TmlPath.FilePath = DeltaGenImportOptionsImpl::FindBestFile(PathNoExt, TEXT("tml"));
	}

	if (TextureDirs.Num() == 0 || !bJustEmptyPaths)
	{
		TArray<FString> Folders = DeltaGenImportOptionsImpl::FindTexturesFolders(PathNoExt);
		TextureDirs.SetNum(Folders.Num());
		for (int32 Index = 0; Index < Folders.Num(); ++Index)
		{
			TextureDirs[Index].Path = Folders[Index];
		}
	}
}

#undef LOCTEXT_NAMESPACE
