// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFBXImportOptions.h"


#include "DatasmithDeltaGenImportOptions.generated.h"

class UDatasmithFBXSceneImportData;

UENUM()
enum class EShadowTextureMode : uint8
{
	Ignore UMETA(Tooltip="Ignore exported shadow textures"),
	AmbientOcclusion UMETA(Tooltip="Use shadow textures as ambient occlusion maps"),
	Multiplier UMETA(Tooltip="Use shadow textures as multipliers for base color and specular"),
	AmbientOcclusionAndMultiplier UMETA(Tooltip="Use shadow textures as ambient occlusion maps as well as multipliers for base color and specular")
};

UCLASS(config = EditorPerProjectUserSettings, HideCategories=(Debug))
class UDatasmithDeltaGenImportOptions : public UDatasmithFBXImportOptions
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Processing, meta = (ToolTip = "Don't keep nodes that marked invisible in FBX(an din the original scene), except switch variants"))
	bool bRemoveInvisibleNodes;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category = Processing, meta = (ToolTip = "Collapse nodes that have identity transform, have no mesh and not used in animation/variants/switches"))
	bool bSimplifyNodeHierarchy;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Import Variants", ToolTip="import VAR files"))
	bool bImportVar;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Variants file path", EditCondition = "bImportVar", ToolTip="Path to the *.var file. By default it will search for a *.var file in the same folder as the FBX file, with the same base filename as it", FilePathFilter="var"))
	FFilePath VarPath;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Import POS States", ToolTip="import POS files"))
	bool bImportPos;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="POS file path", EditCondition = "bImportPos", ToolTip="Path to the *.pos file. By default it will search for a *.pos file in the same folder as the FBX file, with the same base filename as it", FilePathFilter="pos"))
	FFilePath PosPath;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Import TML Animations", ToolTip="import TML files"))
	bool bImportTml;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="TML file path", EditCondition = "bImportTml", ToolTip="Path to the *.tml file. By default it will search for a *.tml file in the same folder as the FBX file, with the same base filename as it", FilePathFilter="tml"))
	FFilePath TmlPath;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Shadow Textures", ToolTip="How to handle shadow textures"))
	EShadowTextureMode ShadowTextureMode;

public:
	void ResetPaths(const FString& InFBXFilename, bool bJustEmptyPaths=true);
};
