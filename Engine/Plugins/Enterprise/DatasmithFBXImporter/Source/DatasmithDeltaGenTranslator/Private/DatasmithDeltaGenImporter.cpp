// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenImporter.h"

#include "Async/AsyncWork.h"
#include "DatasmithAnimationElements.h"
#include "DatasmithDeltaGenAnimationInterpolator.h"
#include "DatasmithDeltaGenImportData.h"
#include "DatasmithDeltaGenImportOptions.h"
#include "DatasmithDeltaGenImporterAuxFiles.h"
#include "DatasmithDeltaGenLog.h"
#include "DatasmithDeltaGenSceneProcessor.h"
#include "DatasmithDeltaGenVariantConverter.h"
#include "DatasmithFBXFileImporter.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithTypes.h"
#include "DatasmithUtils.h"

#include "DatasmithVariantElements.h"
#include "FbxImporter.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "StaticMeshAttributes.h"
#include <core/fbxproperty.h>
#include <scene/fbxdocumentinfo.h>
#include <scene/fbxscene.h>
#include <utils/fbxrenamingstrategyutilities.h>

DEFINE_LOG_CATEGORY(LogDatasmithDeltaGenImport);

#define LOCTEXT_NAMESPACE "DatasmithDeltaGenImporter"

// Use some suffix to make names unique
#define UNIQUE_NAME_SUFFIX TEXT(NAMECLASH1_KEY)

// Do not allow mesh names longer than this value
#define MAX_MESH_NAME_LENGTH 48

// Internally, attachment performed in USceneComponent::AttachToComponent(). This function determines LastAttachIndex
// using some logic, then inserts new actor as FIRST element in child array, i.e. adding actors 1,2,3 will result
// these actors in reverse order (3,2,1). We're using logic which prevents this by iterating children in reverse order.
#define REVERSE_ATTACH_ORDER 1

// Asset paths of blueprints used here
#define SWITCH_ACTOR_CLASS TEXT("/Script/VariantManagerContent.SwitchActor")

class FAsyncReleaseFbxScene : public FNonAbandonableTask
{
public:
	FAsyncReleaseFbxScene( UnFbx::FFbxImporter* InFbxImporter )
		: FbxImporter( InFbxImporter )
	{
	}

	void DoWork()
	{
		FbxImporter->ReleaseScene();
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncReleaseFbxScene, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	UnFbx::FFbxImporter* FbxImporter;
};

FDatasmithDeltaGenImporter::FDatasmithDeltaGenImporter(TSharedRef<IDatasmithScene>& OutScene, UDatasmithDeltaGenImportOptions* InOptions)
	: FDatasmithFBXImporter()
	, DatasmithScene(OutScene)
	, ImportOptions(InOptions)
{
}

FDatasmithDeltaGenImporter::~FDatasmithDeltaGenImporter()
{
}

void FDatasmithDeltaGenImporter::SetImportOptions(UDatasmithDeltaGenImportOptions* InOptions)
{
	ImportOptions = InOptions;
}

bool FDatasmithDeltaGenImporter::OpenFile(const FString& FilePath)
{
	if (!ParseFbxFile(FilePath))
	{
		return false;
	}

	ParseAuxFiles(FilePath);

	FDatasmithFBXScene::FStats Stats = IntermediateScene->GetStats();
	UE_LOGF(LogDatasmithDeltaGenImport, Log, "Scene %ls has %d nodes, %d geometries, %d meshes, %d materials",
		*FilePath, Stats.NodeCount, Stats.GeometryCount, Stats.MeshCount, Stats.MaterialCount);

	ProcessScene();

	Stats = IntermediateScene->GetStats();
	UE_LOGF(LogDatasmithDeltaGenImport, Log, "Processed scene %ls has %d nodes, %d geometries, %d meshes, %d materials",
		*FilePath, Stats.NodeCount, Stats.GeometryCount, Stats.MeshCount, Stats.MaterialCount);


	return true;
}

bool FDatasmithDeltaGenImporter::ParseFbxFile(const FString& FBXPath)
{
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	UnFbx::FBXImportOptions* GlobalImportSettings = FbxImporter->GetImportOptions();
	UnFbx::FBXImportOptions::ResetOptions(GlobalImportSettings);

	if (!FbxImporter->ImportFromFile(FBXPath, FPaths::GetExtension(FBXPath), false))
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Error, "Error parsing FBX file: %ls", FbxImporter->GetErrorMessage());

		( new FAutoDeleteAsyncTask< FAsyncReleaseFbxScene >( FbxImporter ) )->StartBackgroundTask();
		return false;
	}

	FDatasmithImportBaseOptions DefaultBaseOptions;

	FDatasmithFBXFileImporter Importer(FbxImporter->Scene, IntermediateScene.Get(), ImportOptions, &DefaultBaseOptions);
	Importer.ImportScene();

	if ( FbxImporter->Scene && FbxImporter->Scene->GetSceneInfo() )
	{
		DatasmithScene->SetProductName( UTF8_TO_TCHAR( FbxImporter->Scene->GetSceneInfo()->Original_ApplicationName.Get().Buffer() ) );
		DatasmithScene->SetProductVersion( UTF8_TO_TCHAR( FbxImporter->Scene->GetSceneInfo()->Original_ApplicationVersion.Get().Buffer() ) );
		DatasmithScene->SetVendor( UTF8_TO_TCHAR( FbxImporter->Scene->GetSceneInfo()->Original_ApplicationVendor.Get().Buffer() ) );
	}

	( new FAutoDeleteAsyncTask< FAsyncReleaseFbxScene >( FbxImporter ) )->StartBackgroundTask();
	return true;
}

void FDatasmithDeltaGenImporter::UnloadScene()
{
}

void FDatasmithDeltaGenImporter::ParseAuxFiles(const FString& FBXPath)
{
	if (ImportOptions->bImportVar)
	{
		FDatasmithDeltaGenImportVariantsResult VarResult = FDatasmithDeltaGenAuxFiles::ParseVarFile(ImportOptions->VarPath.FilePath);
		for (FName Name : VarResult.SwitchObjects)
		{
			IntermediateScene->SwitchObjects.AddUnique(Name);
		}
		for (FName Name : VarResult.ToggleObjects)
		{
			IntermediateScene->ToggleObjects.AddUnique(Name);
		}
		for (FName Name : VarResult.ObjectSetObjects)
		{
			IntermediateScene->ObjectSetObjects.AddUnique(Name);
		}
		VariantSwitches = VarResult.VariantSwitches;

		// Create a camera actor if we have a camera variant
		for (const FDeltaGenVarDataVariantSwitch& Var : VariantSwitches)
		{
			if (Var.Camera.Variants.Num() > 0)
			{
				TSharedPtr<FDatasmithFBXSceneNode> SceneCameraNode(new FDatasmithFBXSceneNode());
				SceneCameraNode->Name = SCENECAMERA_NAME;
				SceneCameraNode->OriginalName = SCENECAMERA_NAME;
				SceneCameraNode->SplitNodeID = -1;
				SceneCameraNode->LocalTransform = FTransform::Identity;

				TSharedPtr<FDatasmithFBXSceneCamera> SceneCamera = MakeShared<FDatasmithFBXSceneCamera>();
				SceneCameraNode->Camera = SceneCamera;

				IntermediateScene->RootNode->AddChild(SceneCameraNode);
				break;
			}
		}
	}

	if (ImportOptions->bImportPos)
	{
		FDatasmithDeltaGenImportPosResult PosResult = FDatasmithDeltaGenAuxFiles::ParsePosFile(ImportOptions->PosPath.FilePath);
		for (FName Name : PosResult.SwitchObjects)
		{
			IntermediateScene->SwitchObjects.AddUnique(Name);
		}
		for (FName Name : PosResult.SwitchMaterialObjects)
		{
			IntermediateScene->SwitchMaterialObjects.AddUnique(Name);
		}
		PosStates = PosResult.PosStates;
	}

	// TODO: BaseOptions
	if (/*Context.Options->BaseOptions.bIncludeAnimation && */ImportOptions->bImportTml)
	{
		FDatasmithDeltaGenImportTmlResult TmlResult = FDatasmithDeltaGenAuxFiles::ParseTmlFile(ImportOptions->TmlPath.FilePath);
		for (FName Name : TmlResult.AnimatedObjects)
		{
			IntermediateScene->AnimatedObjects.AddUnique(Name);
		}
		TmlTimelines = TmlResult.Timelines;
	}
}

bool FDatasmithDeltaGenImporter::ProcessScene()
{
	FDatasmithDeltaGenSceneProcessor Processor(IntermediateScene.Get());

	Processor.FindDuplicatedMaterials();

	if (ImportOptions->ShadowTextureMode != EShadowTextureMode::Ignore)
	{
		Processor.SetupAOTextures(ImportOptions->TextureDirs);
	}

	Processor.RemoveLightMapNodes();

	Processor.FindPersistentNodes();

	Processor.SplitLightNodes();

	Processor.DecomposePivots(TmlTimelines);

	if (ImportOptions->bRemoveInvisibleNodes)
	{
		Processor.RemoveInvisibleNodes();
	}

	if (ImportOptions->bSimplifyNodeHierarchy)
	{
		Processor.SimplifyNodeHierarchy();
	}

	Processor.FindDuplicatedMeshes();

	Processor.RemoveEmptyNodes();

	Processor.FixMeshNames();

	return true;
}

bool FDatasmithDeltaGenImporter::CheckNodeType(const TSharedPtr<FDatasmithFBXSceneNode>& Node)
{
	if (Node->Mesh.IsValid() && Node->Camera.IsValid())
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Error, "Node '%ls' can't have a mesh and a camera at the same time!", *Node->Name);
		return false;
	}
	else if (Node->Mesh.IsValid() && Node->Light.IsValid())
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Error, "Node '%ls' can't have a mesh and a light at the same time!", *Node->Name);
		return false;
	}
	else if (Node->Light.IsValid() && Node->Camera.IsValid())
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Error, "Node '%ls' can't have a light and a camera at the same time!", *Node->Name);
		return false;
	}

	return true;
}

TSharedPtr<IDatasmithActorElement> FDatasmithDeltaGenImporter::ConvertNode(const TSharedPtr<FDatasmithFBXSceneNode>& Node)
{
	TSharedPtr<IDatasmithActorElement> ActorElement;

	// Check if node can be converted into a datasmith actor
	if (!CheckNodeType(Node))
	{
		return ActorElement;
	}

	if (Node->Mesh.IsValid())
	{
		TSharedPtr<FDatasmithFBXSceneMesh> ThisMesh = Node->Mesh;
		FName MeshName = FName(*ThisMesh->Name);

		TSharedPtr<IDatasmithMeshElement> CreatedMesh = nullptr;
		TSharedPtr<FDatasmithFBXSceneMesh>* FoundMesh = MeshNameToFBXMesh.Find(MeshName);
		if (FoundMesh && (*FoundMesh).IsValid())
		{
			// Meshes should all have unique names by now
			ensure(*FoundMesh == ThisMesh);
		}
		else
		{
			// Create a mesh
			MeshNameToFBXMesh.Add(MeshName, ThisMesh);
			CreatedMesh = FDatasmithSceneFactory::CreateMesh(*ThisMesh->Name);

			FMeshDescription& MeshDescription = ThisMesh->MeshDescription;
			FStaticMeshAttributes StaticMeshAttributes(MeshDescription);
			TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs();
			int32 NumUVChannels = VertexInstanceUVs.GetNumChannels();

			// DeltaGen uses UV channel 0 for texture UVs, and UV channel 1 for lightmap UVs
			// Don't set it to zero or else it will disable Datasmith's GenerateLightmapUV option
			if (NumUVChannels > 1)
			{
				CreatedMesh->SetLightmapCoordinateIndex(1);
			}

			DatasmithScene->AddMesh(CreatedMesh);
		}

		TSharedPtr<IDatasmithMeshActorElement> MeshActorElement = FDatasmithSceneFactory::CreateMeshActor(*Node->Name);
		MeshActorElement->SetStaticMeshPathName(*ThisMesh->Name);

		// Assign materials to the actor
		for (int32 MaterialID = 0; MaterialID < Node->Materials.Num(); MaterialID++)
		{
			TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[MaterialID];
			TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = ConvertMaterial(Material);
			TSharedRef<IDatasmithMaterialIDElement> MaterialIDElement(FDatasmithSceneFactory::CreateMaterialId(MaterialElement->GetName()));
			MaterialIDElement->SetId(MaterialID);
			MeshActorElement->AddMaterialOverride(MaterialIDElement);

			// Also set the material directly on the mesh if this was the node that created it
			if (CreatedMesh)
			{
				CreatedMesh->SetMaterial(MaterialElement->GetName(), MaterialID);
			}
		}

		ActorElement = MeshActorElement;
	}
	else if (Node->Light.IsValid())
	{
		TSharedPtr<IDatasmithLightActorElement> LightActor;
		TSharedPtr<FDatasmithFBXSceneLight> Light = Node->Light;

		// Create the correct type of light and set some type-specific properties. Some others will be set just below this
		// switch, and yet others will be set on post-import, since they're not exposed on the IDatasmithLightActorElement hierarchy
		switch (Light->LightType)
		{
		case ELightType::Point:
		{
			TSharedRef<IDatasmithPointLightElement> PointLight = FDatasmithSceneFactory::CreatePointLight(*Node->Name);
			LightActor = PointLight;
			break;
		}
		case ELightType::Directional:
		{
			TSharedRef<IDatasmithDirectionalLightElement> DirLight = FDatasmithSceneFactory::CreateDirectionalLight(*Node->Name);
			LightActor = DirLight;
			break;
		}
		case ELightType::Spot:
		{
			TSharedRef<IDatasmithSpotLightElement> SpotLight = FDatasmithSceneFactory::CreateSpotLight(*Node->Name);
			SpotLight->SetInnerConeAngle(Light->ConeInnerAngle);
			SpotLight->SetOuterConeAngle(Light->ConeOuterAngle);
			LightActor = SpotLight;
			break;
		}
		case ELightType::Area:
		{
			TSharedRef<IDatasmithAreaLightElement> AreaLight = FDatasmithSceneFactory::CreateAreaLight(*Node->Name);
			AreaLight->SetInnerConeAngle(Light->ConeInnerAngle);
			AreaLight->SetOuterConeAngle(Light->ConeOuterAngle);
			AreaLight->SetLightShape(Light->AreaLightShape);

			if ( !Light->VisualizationVisible )
			{
				AreaLight->SetLightShape( EDatasmithLightShape::None );
			}

			AreaLight->SetWidth(0.2f);
			AreaLight->SetLength(0.2f);

			if (Light->UseIESProfile)
			{
				AreaLight->SetLightType(EDatasmithAreaLightType::IES_DEPRECATED);
			}
			else if (Light->AreaLightUseConeAngle)
			{
				AreaLight->SetLightType(EDatasmithAreaLightType::Spot);
			}
			else
			{
				AreaLight->SetLightType(EDatasmithAreaLightType::Point);
			}

			LightActor = AreaLight;
			break;
		}
		default:
		{
			TSharedRef<IDatasmithLightActorElement> DefaultLight = FDatasmithSceneFactory::CreateAreaLight(*Node->Name);
			LightActor = DefaultLight;
		}
		}

		//Set light units. Only IES-profile based lights seem to use lumens
		if (LightActor->IsA(EDatasmithElementType::PointLight | EDatasmithElementType::AreaLight | EDatasmithElementType::SpotLight))
		{
			IDatasmithPointLightElement* LightAsPointLight = static_cast<IDatasmithPointLightElement*>(LightActor.Get());
			if (LightAsPointLight)
			{
				if (Light->UseIESProfile)
				{
					LightAsPointLight->SetIntensityUnits(EDatasmithLightUnits::Lumens);
				}
				else
				{
					LightAsPointLight->SetIntensityUnits(EDatasmithLightUnits::Candelas);
				}
			}
		}

		LightActor->SetEnabled(Light->Enabled);
		LightActor->SetIntensity(Light->Intensity);
		LightActor->SetColor(Light->DiffuseColor);
		LightActor->SetTemperature(Light->Temperature);
		LightActor->SetUseTemperature(Light->UseTemperature);
		LightActor->SetUseIes(Light->UseIESProfile);
		if (Light->UseIESProfile && !Light->IESPath.IsEmpty())
		{
			// Create IES texture
			const FString BaseFilename = FPaths::GetBaseFilename(Light->IESPath);
			FString TextureName = FDatasmithUtils::SanitizeObjectName(BaseFilename + TEXT("_IES"));
			TSharedPtr<IDatasmithTextureElement> Texture = FDatasmithSceneFactory::CreateTexture(*TextureName);
			Texture->SetTextureMode(EDatasmithTextureMode::Ies);
			Texture->SetLabel(*BaseFilename);
			Texture->SetFile(*Light->IESPath);
			DatasmithScene->AddTexture(Texture);

			// Assign IES texture to light
			LightActor->SetIesTexturePathName(*TextureName);
		}

		ActorElement = LightActor;
	}
	else if (Node->Camera.IsValid())
	{
		TSharedPtr<FDatasmithFBXSceneCamera> Camera = Node->Camera;
		TSharedPtr<IDatasmithCameraActorElement> CameraActor = FDatasmithSceneFactory::CreateCameraActor(*Node->Name);

		CameraActor->SetFocalLength(Camera->FocalLength);
		CameraActor->SetFocusDistance(Camera->FocusDistance);
		CameraActor->SetSensorAspectRatio(Camera->SensorAspectRatio);
		CameraActor->SetSensorWidth(Camera->SensorWidth);

		//We will apply the roll value when splitting the camera node in the scene processor, since
		//we would affect the camera's children otherwise

		ActorElement = CameraActor;
	}
	else
	{
		if (EnumHasAnyFlags(Node->GetNodeType(), ENodeType::Switch))
		{
			TSharedPtr<IDatasmithCustomActorElement> SwitchActorClass = FDatasmithSceneFactory::CreateCustomActor(*Node->Name);
			SwitchActorClass->SetClassOrPathName(SWITCH_ACTOR_CLASS);
			ActorElement = SwitchActorClass;
		}
		else
		{
			// Create regular actor
			ActorElement = FDatasmithSceneFactory::CreateActor(*Node->Name);
		}
	}

	if ( Node->Metadata.Num() > 0 )
	{
		TSharedPtr<IDatasmithMetaDataElement> Metadata = FDatasmithSceneFactory::CreateMetaData( ActorElement->GetName() );
		Metadata->SetAssociatedElement( ActorElement );
		DatasmithScene->AddMetaData( Metadata );

		for ( const TPair<FString, FString>& Pair : Node->Metadata )
		{
			const FString& Key = Pair.Key;
			const FString& Value = Pair.Value;

			TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty( *Key );
			MetadataPropertyPtr->SetPropertyType( EDatasmithKeyValuePropertyType::String );
			MetadataPropertyPtr->SetValue( *Value );
			Metadata->AddProperty( MetadataPropertyPtr );
		}
	}

	ActorElement->AddTag(*Node->OriginalName);
	ActorElement->AddTag(*FString::FromInt(Node->SplitNodeID));

	const FTransform& Transform = Node->GetWorldTransform();
	ActorElement->SetTranslation(Transform.GetTranslation());
	ActorElement->SetScale(Transform.GetScale3D());
	ActorElement->SetRotation(Transform.GetRotation());

#if !REVERSE_ATTACH_ORDER
	for (int32 Index = 0; Index < Node->Children.Num(); Index++)
#else
	for (int32 Index = Node->Children.Num() - 1; Index >= 0; Index--)
#endif
	{
		TSharedPtr<FDatasmithFBXSceneNode>& ChildNode = Node->Children[Index];
		auto ChildNodeActor = ConvertNode(ChildNode);

		if (ChildNodeActor.IsValid())
		{
			ActorElement->AddChild(ChildNodeActor);
		}
	}

	return ActorElement;
}

FORCEINLINE void AddBoolProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, bool Value)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Bool);
	MaterialProperty->SetValue(Value ? TEXT("True") : TEXT("False"));
	Element->AddProperty(MaterialProperty);
}

FORCEINLINE void AddColorProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, const FVector4& Value)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Color);
	FLinearColor Color(Value.X, Value.Y, Value.Z, Value.W);
	MaterialProperty->SetValue(*Color.ToString());
	Element->AddProperty(MaterialProperty);
}

FORCEINLINE void AddFloatProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, float Value)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
	MaterialProperty->SetValue(*FString::SanitizeFloat(Value));
	Element->AddProperty(MaterialProperty);
}

FORCEINLINE void AddStringProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, const FString& Value)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::String);
	MaterialProperty->SetValue(*Value);
	Element->AddProperty(MaterialProperty);
}

FORCEINLINE void AddTextureProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, const FString& Path)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Texture);
	MaterialProperty->SetValue(*Path);
	Element->AddProperty(MaterialProperty);
}

TSharedPtr<IDatasmithTextureElement> CreateTextureAndTextureProperties(IDatasmithMaterialInstanceElement* Element, const FString& TextureName, const FDatasmithFBXSceneMaterial::FTextureParams& Tex, EShadowTextureMode ShadowTextureMode)
{
	using MapEntry = TPairInitializer<const FString&, const EDatasmithTextureMode&>;
	const static TMap<FString, EDatasmithTextureMode> TextureModes
	({
		MapEntry(TEXT("TexBump"), EDatasmithTextureMode::Bump),
		MapEntry(TEXT("TexNormal"), EDatasmithTextureMode::Bump),
		MapEntry(TEXT("TexDiffuse"), EDatasmithTextureMode::Diffuse),
		MapEntry(TEXT("TexSpecular"), EDatasmithTextureMode::Specular),
		MapEntry(TEXT("TexReflection"), EDatasmithTextureMode::Specular),
		MapEntry(TEXT("TexTransparent"), EDatasmithTextureMode::Other),
		MapEntry(TEXT("TexEmissive"), EDatasmithTextureMode::Diffuse),
		MapEntry(TEXT("TexAO"), EDatasmithTextureMode::Diffuse),
		MapEntry(TEXT("TexShininess"), EDatasmithTextureMode::Specular),
	});

	// Create the actual texture (accompanying texture properties will all be packed as key-value pairs)
	FString TexFilename = FPaths::GetCleanFilename(Tex.Path);
	FString TexHandle = TextureName;
	TexHandle[0] = TexHandle.Left(1).ToUpper()[0];

	EDatasmithTextureMode TextureMode = EDatasmithTextureMode::Other;
	if (const EDatasmithTextureMode* FoundMode = TextureModes.Find(TexHandle))
	{
		TextureMode = *FoundMode;
	}
	TSharedRef<IDatasmithTextureElement> DSTexture = FDatasmithSceneFactory::CreateTexture(*FPaths::GetBaseFilename(Tex.Path));
	DSTexture->SetTextureMode(TextureMode);
	DSTexture->SetFile(*Tex.Path);

	// Pack all "texture properties". These are really material properties, but we'll use these to help
	// map the texture correctly. Datasmith will bind these values to the material instance on creation and it will
	// do that by property name, so it is imperative that they are like below (e.g. diffuseTranslation, glossyRotate, etc).
	// Check the reference material graphs to find the matching properties that will be filled in by datasmith
	AddTextureProperty(Element, TexHandle + TEXT("Path"), Tex.Path);

	if (TexHandle == TEXT("TexAO"))
	{
		// Enable usage of shadow texture for ambient occlusion material input
		if (ShadowTextureMode == EShadowTextureMode::AmbientOcclusion || ShadowTextureMode == EShadowTextureMode::AmbientOcclusionAndMultiplier)
		{
			AddBoolProperty(Element, TEXT("TexAOIsActive"), !Tex.Path.IsEmpty());
		}
		// Enable usage of shadow texture as a multiplier on base color and specular
		if (ShadowTextureMode == EShadowTextureMode::Multiplier || ShadowTextureMode == EShadowTextureMode::AmbientOcclusionAndMultiplier)
		{
			AddBoolProperty(Element, TEXT("TexAOAsMultiplier"), !Tex.Path.IsEmpty());
		}
	}
	else
	{
		AddBoolProperty(Element, TexHandle + TEXT("IsActive"), !Tex.Path.IsEmpty());
		AddColorProperty(Element, TexHandle + TEXT("Translation"), Tex.Translation);
		AddColorProperty(Element, TexHandle + TEXT("Rotation"), Tex.Rotation);
		AddColorProperty(Element, TexHandle + TEXT("Scale"), Tex.Scale);
	}

	return DSTexture;
}

namespace DeltaGenImporterImpl
{
	// This will search for a texture with a matching filename first in Path, then in the Textures folder
	FString SearchForFile(FString Path, const TArray<FDirectoryPath>& TextureFolders)
	{
		// The expected behaviour should be that even if the path is correct, if we provide no textures folder
		// it shouldn't import any textures
		if (Path.IsEmpty() || TextureFolders.Num() == 0)
		{
			return FString();
		}

		FPaths::NormalizeFilename(Path);
		if (FPaths::FileExists(Path))
		{
			return Path;
		}

		FString CleanFilename = FPaths::GetCleanFilename(Path);

		for (const FDirectoryPath& TextureFolderDir : TextureFolders)
		{
			const FString& TextureFolder = TextureFolderDir.Path;

			FString InTextureFolder = FPaths::Combine(TextureFolder, CleanFilename);
			if (FPaths::FileExists(InTextureFolder))
			{
				return InTextureFolder;
			}

			// Search recursively inside texture folder
			TArray<FString> FoundFiles;
			IFileManager::Get().FindFilesRecursive(FoundFiles, *TextureFolder, *CleanFilename, true, false);
			if (FoundFiles.Num() > 0)
			{
				return FoundFiles[0];
			}
		}

		return FString();
	}
}

TSharedPtr<IDatasmithBaseMaterialElement> FDatasmithDeltaGenImporter::ConvertMaterial(const TSharedPtr<FDatasmithFBXSceneMaterial>& Material)
{
	TSharedPtr<IDatasmithBaseMaterialElement>* OldMaterial = ImportedMaterials.Find(Material);
	if (OldMaterial != nullptr)
	{
		return *OldMaterial;
	}

	TSharedRef<IDatasmithMaterialInstanceElement> MaterialElement = FDatasmithSceneFactory::CreateMaterialInstance(*Material->Name);
	ImportedMaterials.Add(Material, MaterialElement);

	IDatasmithMaterialInstanceElement* El = &MaterialElement.Get();
	AddStringProperty(El, TEXT("Type"), Material->Type);
	AddBoolProperty(El, TEXT("ReflectionIsActive"), true);

	for (const auto& Pair : Material->TextureParams)
	{
		const FString& TexName = Pair.Key;
		const FDatasmithFBXSceneMaterial::FTextureParams& Tex = Pair.Value;

		FString FoundTexturePath = DeltaGenImporterImpl::SearchForFile(Tex.Path, ImportOptions->TextureDirs);
		if (FoundTexturePath.IsEmpty())
		{
			continue;
		}
		TSharedPtr<IDatasmithTextureElement> CreatedTexture = CreateTextureAndTextureProperties(El, TexName, Tex, ImportOptions->ShadowTextureMode);

		// Only add the texture element to the scene once (so the asset is only created once). We need
		// to let CreateTextureAndTextureProperties run regardless though, as it will add to the material element
		// the properties describing how this material will use this texture
		if (CreatedTexture.IsValid() && !CreatedTextureElementPaths.Contains(FoundTexturePath))
		{
			DatasmithScene->AddTexture(CreatedTexture);
			CreatedTextureElementPaths.Add(FoundTexturePath);
		}
	}

	for (const auto& Pair : Material->BoolParams)
	{
		const FString& ParamName = Pair.Key;
		const bool bValue = Pair.Value;

		AddBoolProperty(El, ParamName, bValue);
	}

	for (const auto& Pair : Material->ScalarParams)
	{
		const FString& ParamName = Pair.Key;
		const float Value = Pair.Value;

		AddFloatProperty(El, ParamName, Value);
	}

	for (const auto& Pair : Material->VectorParams)
	{
		const FString& ParamName = Pair.Key;
		const FVector4& Value = Pair.Value;

		AddColorProperty(El, ParamName, Value);
	}

	return MaterialElement;
}

namespace DeltaGenImporterImpl
{
	void PopulateTransformAnimation(IDatasmithTransformAnimationElement& TransformAnimation, const FDeltaGenTmlDataAnimationTrack& Track, float InFramerate, float TimelineDelayMs)
	{
		EDatasmithTransformType DSType = EDatasmithTransformType::Count;

		switch(Track.Type)
		{
			case EDeltaGenTmlDataAnimationTrackType::Translation:
				DSType = EDatasmithTransformType::Translation;
				break;
			// We convert Rotation to Euler on import
			case EDeltaGenTmlDataAnimationTrackType::Rotation:
			case EDeltaGenTmlDataAnimationTrackType::RotationDeltaGenEuler:
				DSType = EDatasmithTransformType::Rotation;
				break;
			case EDeltaGenTmlDataAnimationTrackType::Scale:
				DSType = EDatasmithTransformType::Scale;
				break;
			case EDeltaGenTmlDataAnimationTrackType::Center:
				UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Center animations are currently not supported!");
				return;
			default:
				return;
		}

		bool bValidValues = false;
		for (const FVector& Value : Track.Values)
		{
			if (Value != FVector::ZeroVector)
			{
				bValidValues = true;
				break;
			}
		}

		bool bValidControlPoints = false;
		for (const FVector& Value : Track.ValueControlPoints)
		{
			if (Value != FVector::ZeroVector)
			{
				bValidControlPoints = true;
				break;
			}
		}

		// Early out if this track has invalid data
		if ((Track.ValueInterpolation == EDeltaGenAnimationInterpolation::Constant ||
			 Track.ValueInterpolation == EDeltaGenAnimationInterpolation::Linear) &&
			!bValidValues)
		{
			return;
		}
		else if (Track.ValueInterpolation == EDeltaGenAnimationInterpolation::Cubic &&
				 (!bValidValues || !bValidControlPoints))
		{
			return;
		}

		// Extract the keys from the time curve control points (x values of tangential control points)
		// DeltaGen will store, for example, these "Positions" for some TimeAdjustment Smooth interpolator:
		//	0.00000000 0.00000000 0.00000000;    <--- P0
		//	5.00000000 5.00000000 0.00000000;    <--- P1
		//	10.00000000 10.00000000 0.00000000;  <--- P2
		//	15.00000000 15.00000000 0.00000000;  <--- P3
		// With this, we will extract 0 and 15 as keys, and have the entire dataset as control points.
		int32 NumKeyControlPoints = Track.KeyControlPoints.Num();
		TArray<float> KeyCurveKeys;

		TUniquePtr<DeltaGen::FInterpolator> KeyInterpolator = nullptr;
		switch (Track.KeyInterpolation)
		{
		case EDeltaGenAnimationInterpolation::Cubic:
			// We should have 1 tangential (actual vertex) and 2 auxiliary (handles) control points per key, except the
			// first and last keys, which have 1 handle less, so NumControlPts = NumKeys + NumKeys * 2 - 2, and so
			// NumKeys = (NumControlPts + 2)/3
			KeyCurveKeys.Reserve((NumKeyControlPoints + 2) / 3);
			for (int32 Index = 0; Index < NumKeyControlPoints; Index += 3)
			{
				KeyCurveKeys.Add(Track.KeyControlPoints[Index].X);
			}

			KeyInterpolator = MakeUnique<DeltaGen::FCubicInterpolator>(KeyCurveKeys, Track.KeyControlPoints);
			break;

		default:
			KeyCurveKeys.Reserve(NumKeyControlPoints);
			for (int32 Index = 0; Index < NumKeyControlPoints; ++Index)
			{
				KeyCurveKeys.Add(Track.KeyControlPoints[Index].X);
			}

			KeyInterpolator = MakeUnique<DeltaGen::FLinearInterpolator>(KeyCurveKeys, Track.KeyControlPoints);
			break;
		}

		TUniquePtr<DeltaGen::FInterpolator> ValueInterpolator = nullptr;
		switch (Track.ValueInterpolation)
		{
		case EDeltaGenAnimationInterpolation::Constant:
			ValueInterpolator = MakeUnique<DeltaGen::FConstInterpolator>(Track.Keys, Track.Values);
			break;
		case EDeltaGenAnimationInterpolation::Linear:
			ValueInterpolator = MakeUnique<DeltaGen::FLinearInterpolator>(Track.Keys, Track.Values);
			break;
		case EDeltaGenAnimationInterpolation::Cubic:
			ValueInterpolator = MakeUnique<DeltaGen::FCubicInterpolator>(Track.Keys, Track.ValueControlPoints);
			break;
		case EDeltaGenAnimationInterpolation::Unsupported:
		default:
			return;
		}

		if (!ValueInterpolator.IsValid() || !KeyInterpolator.IsValid())
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Error, "Unsupported transform animation interpolation type for animation '%ls'!", TransformAnimation.GetLabel());
			return;
		}

		float DelayS = (Track.DelayMs + TimelineDelayMs) / 1000.0f;

		FFrameRate Framerate = FFrameRate(static_cast<uint32>(InFramerate + 0.5f), 1);
		FFrameNumber StartFrame = Framerate.AsFrameNumber(ValueInterpolator->GetMinTime() + DelayS);

		// If we use AsFrameNumber it will floor, and we might lose the very end of the animation
		const double TimeAsFrame = (double(ValueInterpolator->GetMaxTime() + DelayS) * Framerate.Numerator) / Framerate.Denominator;
		FFrameNumber EndFrame = FFrameNumber(static_cast<int32>(FMath::CeilToDouble(TimeAsFrame)));

		// We go to EndFrame.Value+1 here so that if its a 2 second animation at 30fps, frame 60 belongs
		// to the actual animation, as opposed to being range [0, 59]. This guarantees that the animation will
		// actually complete within its range, which is necessary in order to play it correctly at runtime
		for (int32 Frame = StartFrame.Value; Frame <= EndFrame.Value + 1; ++Frame)
		{
			float TimeSeconds = Framerate.AsSeconds(Frame) - DelayS;
			FVector InterpolatedTime = KeyInterpolator->SolveForX(TimeSeconds);
			FVector Val = ValueInterpolator->Evaluate(InterpolatedTime.Y);

			if (DSType == EDatasmithTransformType::Rotation)
			{
				FQuat Xrot = FQuat(FVector(1.0f, 0.0f, 0.0f), FMath::DegreesToRadians(Val.X));
				FQuat Yrot = FQuat(FVector(0.0f, 1.0f, 0.0f), FMath::DegreesToRadians(Val.Y));
				FQuat Zrot = FQuat(FVector(0.0f, 0.0f, 1.0f), FMath::DegreesToRadians(Val.Z));
				Val = (Xrot * Yrot * Zrot).Euler();
				Val.X *= -1;
				Val.Z *= -1;
			}
			else if (DSType == EDatasmithTransformType::Translation)
			{
				// Deltagen is right-handed Z up, UnrealEditor is left-handed Z up. We try keeping the same X, so here we
				// just flip the Y coordinate to convert between them.
				// Note: Geometry, transforms and VRED animations get converted when parsing the FBX file in DatasmithFBXFileImporter, so
				// you won't find analogue for this in VRED importer, even though the conversion is the same
				Val.Y *= -1;
			}

			FDatasmithTransformFrameInfo FrameInfo = FDatasmithTransformFrameInfo(Frame, Val);
			TransformAnimation.AddFrame(DSType, FrameInfo);
		}

		// DeltaGen always has all components for each track type
		EDatasmithTransformChannels Channels = TransformAnimation.GetEnabledTransformChannels();
		ETransformChannelComponents Components = ETransformChannelComponents::All;
		TransformAnimation.SetEnabledTransformChannels(Channels | FDatasmithAnimationUtils::SetChannelTypeComponents(Components, DSType));
	}

	void CombineAnimations(const TArray<TSharedRef<IDatasmithTransformAnimationElement>>& InputAnimations, TSharedRef<IDatasmithTransformAnimationElement>& OutputAnimation)
	{
		TArray<FDatasmithTransformFrameInfo> Frames;
		for (uint8 TransformTypeIndex = 0; TransformTypeIndex < (uint8)EDatasmithTransformType::Count; ++TransformTypeIndex)
		{
			EDatasmithTransformType TransformType = (EDatasmithTransformType)TransformTypeIndex;

			int32 NumFrames = 0;
			for (const TSharedRef<IDatasmithTransformAnimationElement>& Animation : InputAnimations)
			{
				NumFrames += Animation->GetFramesCount(TransformType);
			}
			Frames.Reset();
			Frames.Reserve(NumFrames);

			for (const TSharedRef<IDatasmithTransformAnimationElement>& Animation : InputAnimations)
			{
				int32 FrameCount = Animation->GetFramesCount(TransformType);
				for (int32 FrameIndex = 0; FrameIndex < FrameCount; ++FrameIndex)
				{
					Frames.Add(Animation->GetFrame(TransformType, FrameIndex));
				}
			}

			Algo::Sort(Frames, [](const FDatasmithTransformFrameInfo& A, const FDatasmithTransformFrameInfo& B)
			{
				return A.FrameNumber < B.FrameNumber;
			});

			for (const FDatasmithTransformFrameInfo& Frame : Frames)
			{
				OutputAnimation->AddFrame(TransformType, Frame);
			}
		}
	}
}

TSharedPtr<IDatasmithLevelSequenceElement> FDatasmithDeltaGenImporter::ConvertAnimationTimeline(const FDeltaGenTmlDataTimeline& TmlTimeline)
{
	TSharedRef<IDatasmithLevelSequenceElement> SequenceElement = FDatasmithSceneFactory::CreateLevelSequence(*TmlTimeline.Name);
	SequenceElement->SetFrameRate(TmlTimeline.Framerate);

	TArray<TSharedRef<IDatasmithTransformAnimationElement>> InitialAnimations;
	for (const FDeltaGenTmlDataTimelineAnimation& Animation : TmlTimeline.Animations)
	{
		FString TargetNodeName = Animation.TargetNode.ToString();

		// DeltaGen has no subsequence animations, they're all all Transform
		TSharedRef<IDatasmithTransformAnimationElement> TransformAnimation = FDatasmithSceneFactory::CreateTransformAnimation(*TargetNodeName);
		TransformAnimation->SetEnabledTransformChannels(EDatasmithTransformChannels::None);

		for (const FDeltaGenTmlDataAnimationTrack& Track : Animation.Tracks)
		{
			DeltaGenImporterImpl::PopulateTransformAnimation(TransformAnimation.Get(), Track, TmlTimeline.Framerate, Animation.DelayMs);
		}

		if (TransformAnimation->GetFramesCount(EDatasmithTransformType::Translation) > 0 ||
			TransformAnimation->GetFramesCount(EDatasmithTransformType::Rotation) > 0 ||
			TransformAnimation->GetFramesCount(EDatasmithTransformType::Scale) > 0)
		{
			InitialAnimations.Add(TransformAnimation);
		}
	}

	// Temp fix for UE-87458 until DatasmithLevelSequenceImporter can handle multiple simultaneous animations per actor using separate sections.
	// This combines animations so that we will only ever emit one IDatasmithTransformAnimationElement per actor per level sequence.
	// This needs to be done after baking as tracks can have different interpolation modes.
	{
		TMap<FString, TArray<TSharedRef<IDatasmithTransformAnimationElement>>> AnimationsPerActor;

		for (const TSharedRef<IDatasmithTransformAnimationElement>& Animation : InitialAnimations)
		{
			AnimationsPerActor.FindOrAdd(Animation->GetName()).Add(Animation);
		}

		for (const TPair<FString, TArray<TSharedRef<IDatasmithTransformAnimationElement>>>& Pair : AnimationsPerActor)
		{
			const FString& ActorName = Pair.Key;
			const TArray<TSharedRef<IDatasmithTransformAnimationElement>>& Animations = Pair.Value;

			if (Animations.Num() == 1)
			{
				SequenceElement->AddAnimation(Animations[0]);
			}
			else
			{
				TSharedRef<IDatasmithTransformAnimationElement> CombinedAnimation = FDatasmithSceneFactory::CreateTransformAnimation(*ActorName);
				DeltaGenImporterImpl::CombineAnimations(Animations, CombinedAnimation);

				SequenceElement->AddAnimation(CombinedAnimation);
			}
		}
	}

	return SequenceElement;
}

struct FNameDuplicateFinder
{
	TMap<FString, int32> NodeNames;
	TMap<FString, int32> MeshNames;
	TMap<FString, int32> MaterialNames;

	TSet< TSharedPtr<FDatasmithFBXSceneMesh> > ProcessedMeshes;
	TSet< TSharedPtr<FDatasmithFBXSceneMaterial> > ProcessedMaterials;

	void MakeUniqueName(FString& Name, TMap<FString, int32>& NameList)
	{
		// We're using lowercase name value to make NameList case-insensitive. These names
		// should be case-insensitive because uasset file names directly depends on them.
		FString LowercaseName = Name.ToLower();
		int32* LastValue = NameList.Find(LowercaseName);
		if (LastValue == nullptr)
		{
			// Simplest case: name is not yet used
			NameList.Add(LowercaseName, 0);
			return;
		}

		// Append a numeric suffix
		int32 NameIndex = *LastValue + 1;
		FString NewName;
		do
		{
			NewName = FString::Printf(TEXT("%s%s%d"), *Name, UNIQUE_NAME_SUFFIX, NameIndex);
		} while (NameList.Contains(NewName));

		// Remember the new name
		*LastValue = NameIndex;
		NameList.Add(NewName.ToLower());
		Name = NewName;
	}

	void ResolveDuplicatedObjectNamesRecursive(TSharedPtr<FDatasmithFBXSceneNode>& Node)
	{
		// Process node name
		MakeUniqueName(Node->Name, NodeNames);

		// Process mesh name
		TSharedPtr<FDatasmithFBXSceneMesh>& Mesh = Node->Mesh;
		if (Mesh.IsValid() && !ProcessedMeshes.Contains(Mesh))
		{
			if (Mesh->Name.Len() > MAX_MESH_NAME_LENGTH)
			{
				// Truncate the mesh name if it is too long
				FString NewName = Mesh->Name.Left(MAX_MESH_NAME_LENGTH - 3) + TEXT("_tr");
				UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Mesh name '%ls' is too long, renaming to '%ls'", *Mesh->Name, *NewName);
				Mesh->Name = NewName;
			}

			MakeUniqueName(Mesh->Name, MeshNames);
			ProcessedMeshes.Add(Mesh);
		}

		// Process material names
		for (int32 MaterialIndex = 0; MaterialIndex < Node->Materials.Num(); MaterialIndex++)
		{
			TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[MaterialIndex];
			if (Material.IsValid() && !ProcessedMaterials.Contains(Material))
			{
				MakeUniqueName(Material->Name, MaterialNames);
				ProcessedMaterials.Add(Material);
			}
		}

		for (int32 ChildIndex = 0; ChildIndex < Node->Children.Num(); ChildIndex++)
		{
			ResolveDuplicatedObjectNamesRecursive(Node->Children[ChildIndex]);
		}
	}
};

bool FDatasmithDeltaGenImporter::SendSceneToDatasmith()
{
	if (!IntermediateScene->RootNode.IsValid())
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Error, "FBX Scene root is invalid!");
		return false;
	}
	// Ensure nodes, meshes and materials has unique names
	FNameDuplicateFinder NameDupContext;
	NameDupContext.ResolveDuplicatedObjectNamesRecursive(IntermediateScene->RootNode);

	// Perform conversion
	TSharedPtr<IDatasmithActorElement> NodeActor = ConvertNode(IntermediateScene->RootNode);

	if (NodeActor.IsValid())
	{
		// We need the root node as that is what carries the scaling factor conversion
		DatasmithScene->AddActor(NodeActor);

		// make sure all materials are passed to DS eveno those not used on scene meshes(for material switching)
		for(const TSharedPtr<FDatasmithFBXSceneMaterial>& Material: IntermediateScene->Materials)
		{
			TSharedPtr<IDatasmithBaseMaterialElement> ConvertedMat = ConvertMaterial(Material);
			DatasmithScene->AddMaterial(ConvertedMat);
		}

		// Note: Unlike VRED, DeltaGen does not pack any animations directly to the FBX file, so there
		// is no point in checking DeltaGenScene->AnimNodes

		// Theoretically we can have animations without spawning a scene actor, but if that
		// failed we won't have any actors we can target anyway, so all the sequences will be empty
		for (FDeltaGenTmlDataTimeline& Timeline : TmlTimelines)
		{
			TSharedPtr<IDatasmithLevelSequenceElement> ConvertedSequence = ConvertAnimationTimeline(Timeline);
			if (ConvertedSequence.IsValid())
			{
				DatasmithScene->AddLevelSequence(ConvertedSequence.ToSharedRef());
			}
		}

		TMap<FName, TArray<TSharedPtr<IDatasmithActorElement>>> ImportedActorsByOriginalName;
		TMap<FName, TSharedPtr<IDatasmithBaseMaterialElement>> ImportedMaterialsByName;
		BuildAssetMaps(DatasmithScene, ImportedActorsByOriginalName, ImportedMaterialsByName);

		if (ImportOptions->bImportVar)
		{
			TSharedPtr<IDatasmithLevelVariantSetsElement> LevelVariantSets = FDeltaGenVariantConverter::ConvertVariants(VariantSwitches, PosStates, ImportedActorsByOriginalName, ImportedMaterialsByName);
			if (LevelVariantSets.IsValid())
			{
				DatasmithScene->AddLevelVariantSets(LevelVariantSets.ToSharedRef());
			}
		}
	}
	else
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Error, "Root node '%ls' failed to convert!", *IntermediateScene->RootNode->Name);
		return false;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
