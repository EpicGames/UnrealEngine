// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFBXImporter.h"

struct FDeltaGenPosDataState;
struct FDeltaGenTmlDataTimeline;
struct FDeltaGenVarDataVariantSwitch;

class IDatasmithActorElement;
class IDatasmithBaseMaterialElement;
class IDatasmithLevelSequenceElement;
class IDatasmithScene;
class UDatasmithDeltaGenImportOptions;
struct FDatasmithFBXSceneMaterial;
struct FDatasmithFBXSceneNode;

class FDatasmithDeltaGenImporter : public FDatasmithFBXImporter
{
public:
	FDatasmithDeltaGenImporter(TSharedRef<IDatasmithScene>& OutScene, UDatasmithDeltaGenImportOptions* InOptions);
	virtual ~FDatasmithDeltaGenImporter();

	/** Updates the used import options to InOptions */
	void SetImportOptions(UDatasmithDeltaGenImportOptions* InOptions);

	/** Open and load a scene file. */
	bool OpenFile(const FString& FilePath);

	/** Finalize import of the DeltaGen scene into the engine */
	bool SendSceneToDatasmith();

	/** Clean up any unused memory or data. */
	void UnloadScene();

private:
	/** Open and load a DeltaGen .fbx scene file, placing its data in the context */
	bool ParseFbxFile(const FString& FBXPath);

	/** Open auxilliary .lights, .clips and/or .var files exported from VRED into the context */
	void ParseAuxFiles(const FString& FBXPath);

	/** Perform processing of internal scene data for better performance after import */
	bool ProcessScene();

	/** Checks if the NodeType flag has a combination supported for this plugin */
	bool CheckNodeType(const TSharedPtr<FDatasmithFBXSceneNode>& Node);

	TSharedPtr<IDatasmithActorElement> ConvertNode(const TSharedPtr<FDatasmithFBXSceneNode>& Node);

	TSharedPtr<IDatasmithBaseMaterialElement> ConvertMaterial(const TSharedPtr<FDatasmithFBXSceneMaterial>& Material);

	TSharedPtr<IDatasmithLevelSequenceElement> ConvertAnimationTimeline(const FDeltaGenTmlDataTimeline& TmlTimeline);

private:
	/** Output Datasmith scene */
	TSharedRef<IDatasmithScene> DatasmithScene;

	/** All options to configure importer */
	const UDatasmithDeltaGenImportOptions* ImportOptions;

	// Used to prevent us from needlessly importing the same texture multiple times
	TSet<FString> CreatedTextureElementPaths;

	// Things parsed from aux files
	TArray<FDeltaGenVarDataVariantSwitch> VariantSwitches;
	TArray<FDeltaGenPosDataState> PosStates;
	TArray<FDeltaGenTmlDataTimeline> TmlTimelines;
};
