// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenImporterAuxFiles.h"

#include "DatasmithDeltaGenImportData.h"
#include "DatasmithDeltaGenLog.h"
#include "DatasmithUtils.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "ObjectTools.h"
#include "XmlFile.h"
#include "XmlNode.h"

namespace DeltaGenAuxFiles
{
	// Optimize this statement: for(const FXmlNode* Node = ParentNode->GetFirstChildNode();Node; Node = Node->GetNextNode())
	// into this: for(const FXmlNode* Node: FXmlNodeChildren(ParentNode))
	class FXmlNodeChildren
	{
		const FXmlNode* Node;
	public:
		class Iterator{
			const FXmlNode* Node;
		public:
			Iterator(const FXmlNode* Node): Node(Node) {}

			const Iterator& operator++()
			{
				Node = Node->GetNextNode();
				return *this;
			}

			bool operator!=(const Iterator& other)
			{
				return Node != other.Node;
			}

			const FXmlNode* operator*()
			{
				return Node;
			}
		};

		FXmlNodeChildren(const FXmlNode* Node): Node(Node) {}

		const Iterator begin(){
			return Iterator(Node->GetFirstChildNode());
		}
		const Iterator end(){
			return Iterator(nullptr);
		}
	};

	/** Converts DG Euler in RADIANS to FRotator */
	FRotator ConvertDeltaGenEulerToRotator(float X, float Y, float Z)
	{
		// DeltaGen Euler triple has different order for XYZ transforms than UE
		FRotator Rotator = (FQuat(FVector(1, 0, 0), X)*FQuat(FVector(0, 1, 0), Y)*FQuat(FVector(0, 0, 1), Z)).Rotator();
		return FRotator(Rotator.Pitch, -Rotator.Yaw, -Rotator.Roll);
	}

	enum EDGProperty
	{
		EDG_Unknown,
		EDG_Visibility,
		EDG_ModelTranslation,
		EDG_ModelRotation,
		EDG_VariantID,
		EDG_ActiveChild,
		EDG_Translation,
		EDG_Rotation,
		EDG_Scaling,
		EDG_Center,

		EDG_Count
	};

	const TCHAR* DGPropertyNames[] =
	{
		TEXT("UNKNOWN"),
		TEXT("P:VISIBILITY_FLAG"),
		TEXT("P:MODEL_TRANSLATION"),
		TEXT("P:MODEL_ROTATION"),
		TEXT("P:VARIANT_ID"),
		TEXT("P:ACTIVE_CHILD"),
		TEXT("P:TRANSLATION"),
		TEXT("P:ROTATION"),
		TEXT("P:SCALING"),
		TEXT("P:CENTER"),
	};

	int32 GetTargetID(const FXmlNode* Parent)
	{
		const FXmlNode* TargetIdNode = Parent->FindChildNode(TEXT("TargetID"));
		if (!TargetIdNode)
		{
			return -1;
		}
		return FCString::Atoi(*TargetIdNode->GetContent());
	}

	EDGProperty PropertyIDEnum(const FString& PropertyID)
	{
		static_assert(UE_ARRAY_COUNT(DGPropertyNames) == EDG_Count, "Incomplete DGPropertyNames values");

		for (int32 Index = 0; Index < EDG_Count; Index++)
		{
			if (PropertyID == DGPropertyNames[Index])
			{
				return (EDGProperty) Index;
			}
		}

		return EDG_Unknown;
	}

	FString DatasmitDeltaGenSanitizeObjectName(FString InString)
	{
		const FString Invalid = TEXT("+*\\<>?-"); // these chars are coming through FBX import replaced by underscore

		return FDatasmithUtils::SanitizeObjectName(ObjectTools::SanitizeInvalidChars(InString, Invalid));
	}

	// fixes xml to prepare to load it with UE FastXml
	void FixDeltaGenXml(TArray<FString>& FileContentLines)
	{
		for (auto& Line : FileContentLines)
		{
			// fix non-conformant XML that DG produces
			Line.ReplaceInline(TEXT("&lt;"), TEXT("<")); // fixes metaDataQuery
			Line.ReplaceInline(TEXT("DAF::"), TEXT("DAF_")); // fixes DAF::AttributeType


			// replace text contents of Name/name tag with spaces changed to "_" underscore
			// this fixes problem with FastXML that changes all sequences of spaces to single space
			// which led to mismatching node names referenced in XML with node names imported from
			// FBX(where all space sequences kept, having each space converted to "_")
			const FString InvalidChars = TEXT("<>&' "); // those characters are replaced by underscore anyway in FBX, so replace them together with space
			{
				FString NameTagOpen = "<Name>";
				FString NameTagClose = "</Name>";
				if (Line.TrimStart().StartsWith(NameTagOpen, ESearchCase::IgnoreCase))
				{
					int32 NameTagOpenIndex = Line.Find(NameTagOpen, ESearchCase::IgnoreCase);

					int32 NameTagCloseIndex = Line.Find(NameTagClose, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

					if (NameTagOpenIndex >= 0 && NameTagOpenIndex < NameTagCloseIndex)
					{
						int32 NameTagOpenEndIndex = NameTagOpenIndex + NameTagOpen.Len();
						FString NameTextContent = Line.Mid(NameTagOpenEndIndex, NameTagCloseIndex - NameTagOpenEndIndex);
						Line = Line.Mid(0, NameTagOpenEndIndex) + ObjectTools::SanitizeInvalidChars(NameTextContent, InvalidChars) + Line.Mid(NameTagCloseIndex);
					}
				}
			}

			// same whitespace fix but in lines like this -
			// <value DAF::AttributeType="SingleString"><metaDataQuery version="1"><constraintGroup operator="or"><objectNameConstraint operator="equal" value="Hello   World" ...
			FString ValueDecl = TEXT("value=\"");
			if (Line.Contains(TEXT("metaDataQuery")) && Line.Contains(ValueDecl))
			{
				int32 ValueIndex = Line.Find(ValueDecl);
				int32 NameStart = ValueIndex + ValueDecl.Len();
				int32 NameClosingQuoteIndex = Line.Find(TEXT("\""), ESearchCase::IgnoreCase, ESearchDir::FromStart, NameStart);

				if (NameClosingQuoteIndex > ValueIndex)
				{
					FString NameTextContent = Line.Mid(NameStart, NameClosingQuoteIndex - NameStart);
					Line = Line.Mid(0, NameStart) + ObjectTools::SanitizeInvalidChars(NameTextContent, InvalidChars) + Line.Mid(NameClosingQuoteIndex);
				}
			}


		}
	}

	bool LoadVARFile(const TCHAR* InFilePath, FDatasmithDeltaGenImportVariantsResult& OutResult)
	{
		FXmlFile VarFile;

		TArray<FString> FileContentLines;
		if(!FFileHelper::LoadFileToStringArray(FileContentLines, InFilePath))
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Error, "Couldn't read VAR file: %ls", InFilePath);
		}

		FixDeltaGenXml(FileContentLines);

		// it would have been nice to have FXmlFile to expose LoadFile for lines(it splits into lines anyway inside)
		FString FileContent = FString::Join(FileContentLines, TEXT("\n"));

		if (!VarFile.LoadFile(FileContent, EConstructMethod::ConstructFromBuffer))
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Error, "Couldn't open VAR file: %ls", *VarFile.GetLastError());
			return false;
		}

		const FXmlNode* RootNode = VarFile.GetRootNode();
		const FXmlNode* ProductAspectsNode = RootNode->FindChildNode("ProductAspects");
		if (!ProductAspectsNode)
		{
			return false;
		}

		const FXmlNode* AspectContainerNode = ProductAspectsNode->FindChildNode("AspectContainer");
		if (!AspectContainerNode)
		{
			return false;
		}

		const FXmlNode* VariantSwitchAspectNode = nullptr;
		for(const FXmlNode* Node: FXmlNodeChildren(AspectContainerNode))
		{
			if ("Aspect" == Node->GetTag() && "VariantSwitch" == Node->GetAttribute("Type"))
			{
				VariantSwitchAspectNode = Node;
				break;
			}
		}

		if (!VariantSwitchAspectNode)
		{
			return false;
		}

		for(const FXmlNode* Node: FXmlNodeChildren(VariantSwitchAspectNode))
		{
			if ("VariantSwitch" != Node->GetTag())
			{
				continue;
			}

			const FXmlNode* VariantSwitchNode = Node;

			FDeltaGenVarDataVariantSwitch* VariantSwitch = new(OutResult.VariantSwitches) FDeltaGenVarDataVariantSwitch;

			const FXmlNode* PrototypeNode = VariantSwitchNode->FindChildNode("PrototypeID");
			if (!PrototypeNode)
			{
				continue;
			}
			const FString& Prototype = PrototypeNode->GetContent();

			const FXmlNode* NameNode = VariantSwitchNode->FindChildNode("Name");
			if (NameNode)
			{
				VariantSwitch->Name = NameNode->GetContent();
			}

			EDeltaGenVarDataVariantSwitchType VariantSetType = EDeltaGenVarDataVariantSwitchType::Unsupported;

			if (Prototype == TEXT("GEOMETRY_VARIANT"))
			{
				VariantSetType = EDeltaGenVarDataVariantSwitchType::Geometry;
			}
			else if (Prototype == TEXT("INDIVIDUAL_SCENE_OBJECT_VARIANT_ID"))
			{
				VariantSetType = EDeltaGenVarDataVariantSwitchType::ObjectSet;
			}
			else if (Prototype == TEXT("CAMERA_VARIANT_ID"))
			{
				VariantSetType = EDeltaGenVarDataVariantSwitchType::Camera;
			}
			else if (Prototype == TEXT("PACKAGE_VARIANT"))
			{
				VariantSetType = EDeltaGenVarDataVariantSwitchType::Package;
			}
			else if (Prototype == TEXT("LINKED_SWITCH_OBJECT_VARIANT_ID"))
			{
				VariantSetType = EDeltaGenVarDataVariantSwitchType::SwitchObject;
			}

			VariantSwitch->Type = VariantSetType;

			const FXmlNode* TargetLists = VariantSwitchNode->FindChildNode("TargetLists");
			if (!TargetLists)
			{
				continue;
			}

			TMap<int32, FName> TargetIDToSanitizedName;
			TMap<int32, FString> TargetIDToName;
			TMap<FString, int32> TargetNameToID;

			for (const FXmlNode* TargetDescNode: FXmlNodeChildren(TargetLists))
			{
				if ("TargetDescription" != TargetDescNode->GetTag())
				{
					continue;
				}

				const FXmlNode* PrototypeIdNode = TargetDescNode->FindChildNode("prototypeId");
				if (!PrototypeIdNode)
				{
					continue;
				}

				const FString& TargetPrototype = PrototypeIdNode->GetContent();

				const FXmlNode* TargetNameNode = TargetDescNode->FindChildNode("name");
				if (!TargetNameNode)
				{
					continue;
				}
				const FString& Name = TargetNameNode->GetContent();

				int32 TargetID = GetTargetID(TargetDescNode);
				if (TargetID < 0)
				{
					continue;
				}

				// make sure we use same names as Datasmith scene nodes
				// !!! for some reason dash also replaced in name, although SanitizeObjectName doesn't do it...
				TargetIDToSanitizedName.Add(TargetID, FName(*DatasmitDeltaGenSanitizeObjectName(Name)));
				TargetIDToName.Add(TargetID, Name);
				TargetNameToID.Add(Name, TargetID);
			}

			int32 PropertyIndex = -1;
			struct FPropertyMap
			{
				int32	PropertyIndex;
				int32	TargetID;
				EDGProperty	PropertyID;
			};
			TArray<FPropertyMap> PropertyMap;

			for (const FXmlNode* TargetListNode: FXmlNodeChildren(TargetLists))
			{
				if ("TargetList" != TargetListNode->GetTag())
				{
					continue;
				}
				PropertyIndex++;

				const FXmlNode* TargetNode = TargetListNode->FindChildNode("Target");
				if (!TargetNode)
				{
					continue;
				}
				const FXmlNode* PropertyIDNode = TargetNode->FindChildNode("PropertyID");
				if (!PropertyIDNode)
				{
					continue;
				}
				int32 TargetID = GetTargetID(TargetNode);
				if (TargetID < 0)
				{
					continue;
				}

				EDGProperty PropertyID = PropertyIDEnum(PropertyIDNode->GetContent());
				// verify if we're tracking this target
				if (TargetIDToName.Find(TargetID) == nullptr)
				{
					continue;
				}

				// map property name to enum and remember its index
				if (PropertyID != EDG_Unknown)
				{
					FPropertyMap* PropMap = new(PropertyMap) FPropertyMap;
					PropMap->PropertyIndex = PropertyIndex;
					PropMap->TargetID = TargetID;
					PropMap->PropertyID = PropertyID;
				}
			}

			int32 PropertyCount = PropertyIndex+1;

			// Process <VariantList>
			const FXmlNode* VariantList = VariantSwitchNode->FindChildNode(TEXT("VariantList"));
			if (VariantList)
			{
				VariantList = VariantList->FindChildNode(TEXT("Variants"));
			}

			if (!VariantList)
			{
				continue;
			}

			TArray<FName> TargetNodeNameForProperties;

			switch(VariantSetType)
			{
			case EDeltaGenVarDataVariantSwitchType::Geometry:
				{
					VariantSwitch->Geometry.TargetNodes.AddZeroed(PropertyCount);
					for (auto& prop: PropertyMap) //-V1078
					{
						// use name from var file for package variant names(sanitized only used for scene nodes)
						VariantSwitch->Geometry.TargetNodes[prop.PropertyIndex] = TargetIDToSanitizedName[prop.TargetID];
					}
				}
				break;
			case EDeltaGenVarDataVariantSwitchType::ObjectSet:
				{
					TargetNodeNameForProperties.SetNumZeroed(PropertyCount);
					for (auto& prop: PropertyMap) //-V1078
					{
						TargetNodeNameForProperties[prop.PropertyIndex] = TargetIDToSanitizedName[prop.TargetID];
					}
				}
				break;
			case EDeltaGenVarDataVariantSwitchType::Camera:
				break;
			case EDeltaGenVarDataVariantSwitchType::Package:
				{
					VariantSwitch->Package.TargetVariantSets.AddZeroed(PropertyCount);
					for (auto& prop: PropertyMap) //-V1078
					{
						// use name from var file for package variant names(sanitized only used for scene nodes)
						VariantSwitch->Package.TargetVariantSets[prop.PropertyIndex] = TargetIDToName[prop.TargetID];
					}
				}
				break;
			case EDeltaGenVarDataVariantSwitchType::SwitchObject:
				{
					for (auto& prop: PropertyMap) //-V1078
					{
						if (prop.PropertyID == EDG_ActiveChild)
						{
							VariantSwitch->SwitchObject.TargetSwitchObject = TargetIDToSanitizedName[prop.TargetID];
						}
					}
				}
				break;
			}

			int32 VariantIndex = -1;
			for (const FXmlNode* VariantNode: FXmlNodeChildren(VariantList))
			{
				if (VariantNode->GetTag() != TEXT("Variant"))
				{
					continue;
				}

				VariantIndex++;

				FString VariantName;
				const FXmlNode* VariantNameNode = VariantNode->FindChildNode(TEXT("Name"));
				if (VariantNameNode)
				{
					VariantName = VariantNameNode->GetContent();
				}

				int32 VariantID = -1;
				const FXmlNode* VariantIDNode = VariantNode->FindChildNode("VariantID");
				if (VariantIDNode)
				{
					VariantID = FCString::Atoi(*VariantIDNode->GetContent());
				}

				VariantSwitch->VariantIDToVariantIndex.Add(VariantID, VariantIndex);
				VariantSwitch->VariantIDToVariantName.Add(VariantID, VariantName);

				FDeltaGenVarDataGeometryVariant* MeshVariant = nullptr;
				FDeltaGenVarDataCameraVariant* CameraVariant = nullptr;
				FDeltaGenVarDataPackageVariant* PackageVariant = nullptr;
				FDeltaGenVarDataSwitchObjectVariant* SwitchObjectVariant = nullptr;
				FDeltaGenVarDataObjectSetVariant* ObjectSetVariant = nullptr;
				switch(VariantSetType)
				{
				case EDeltaGenVarDataVariantSwitchType::Geometry:
					{
						MeshVariant = new(VariantSwitch->Geometry.Variants) FDeltaGenVarDataGeometryVariant;
						MeshVariant->Name = VariantName;
					}
					break;
				case EDeltaGenVarDataVariantSwitchType::ObjectSet:
					{
						ObjectSetVariant = new(VariantSwitch->ObjectSet.Variants) FDeltaGenVarDataObjectSetVariant;
						ObjectSetVariant->Name = VariantName;
					}
					break;
				case EDeltaGenVarDataVariantSwitchType::Camera:
					{
						CameraVariant = new(VariantSwitch->Camera.Variants) FDeltaGenVarDataCameraVariant;
						CameraVariant->Name = VariantName;
					}
					break;
				case EDeltaGenVarDataVariantSwitchType::Package:
					{
						PackageVariant = new(VariantSwitch->Package.Variants) FDeltaGenVarDataPackageVariant;
						PackageVariant->Name = VariantName;
					}
					break;
				case EDeltaGenVarDataVariantSwitchType::SwitchObject:
					{
						SwitchObjectVariant = new(VariantSwitch->SwitchObject.Variants) FDeltaGenVarDataSwitchObjectVariant;
						SwitchObjectVariant->Name = VariantName;
					}
					break;
				}

				// CAMERA_VARIANT_ID uses Values node to store camera params
				// INDIVIDUAL_SCENE_OBJECT_VARIANT_ID - every value in variant corresponds to property(TargetList) in targetlists so this variant essentially sets different parameters for objects
				// LINKED_SWITCH_OBJECT_VARIANT_ID - selects switch child there
				if (const FXmlNode* ValuesNode = VariantNode->FindChildNode(TEXT("Values")))
				{
					PropertyIndex = -1;
					for (const FXmlNode* ValueNode: FXmlNodeChildren(ValuesNode))
					{
						if (ValueNode->GetTag() != TEXT("Value"))
						{
							continue;
						}
						PropertyIndex++;
						// Find this value by PropertyIndex in PropertyMap
						FPropertyMap* Prop = PropertyMap.FindByPredicate([PropertyIndex](const FPropertyMap& P)
						{
							return P.PropertyIndex == PropertyIndex;
						});

						if (!Prop)
						{
							// We're not tracking this property
							continue;
						}

						const FXmlNode* DataNode = ValueNode->FindChildNode(TEXT("Data"));
						if (CameraVariant && DataNode)
						{
							if (Prop->PropertyID == EDG_ModelTranslation)
							{
								float X = FCString::Atof(*DataNode->FindChildNode(TEXT("x"))->GetContent());
								float Y = FCString::Atof(*DataNode->FindChildNode(TEXT("y"))->GetContent());
								float Z = FCString::Atof(*DataNode->FindChildNode(TEXT("z"))->GetContent());
								FVector Location = FVector(X, -Y, Z);
								CameraVariant->Location = Location;
							}
							if (Prop->PropertyID == EDG_ModelRotation)
							{

								{
									const FXmlNode* Euler = DataNode->FindChildNode(TEXT("Euler"));
									float X = FCString::Atof(*Euler->FindChildNode(TEXT("x"))->GetContent());
									float Y = FCString::Atof(*Euler->FindChildNode(TEXT("y"))->GetContent());
									float Z = FCString::Atof(*Euler->FindChildNode(TEXT("z"))->GetContent());

									// just convert ueler to quaternion orientation - we don't need excess angle values information(360 dergree) that euler has
									// for camera orientation, quat will suffice
									CameraVariant->Rotation = ConvertDeltaGenEulerToRotator(X, Y, Z);
								}

								// DG also exports rotation as axis/angle for some reason in addition to euler angles
								// leaving code here in case we see some examples where eulers are not present?
								#if (0) 
								{
									const FXmlNode* AxisNode = DataNode->FindChildNode(TEXT("Axis"));
									float X = FCString::Atof(*AxisNode->FindChildNode(TEXT("x"))->GetContent());
									float Y = FCString::Atof(*AxisNode->FindChildNode(TEXT("y"))->GetContent());
									float Z = FCString::Atof(*AxisNode->FindChildNode(TEXT("z"))->GetContent());

									FVector Axis(X, -Y, Z); // convert vector from DG coordinate system to UE
									//CameraVariant->RotationAxis = Axis;

									const FXmlNode* AngleNode = DataNode->FindChildNode(TEXT("Angle"));
									float Angle = -FCString::Atof(*AngleNode->GetContent()); // negate rotation to convert from DG to UE
									//CameraVariant->RotationAngle = Angle;

									FVector SafeAxis = Axis.GetSafeNormal();
									//CameraVariant->Rotation = FQuat(SafeAxis, Angle).Rotator();
								}
								#endif
							}
						}

						if (MeshVariant && DataNode && (Prop->PropertyID == EDG_Visibility))
						{
							const FString& DataStr = DataNode->GetContent();
							// Note: the same mesh name could appear many times in the case if the same
							// object appears as instance - avoid adding the same name again. Use AddUnique()
							// to ensure that.
							bool bVisible = FCString::Atoi(*DataStr) != 0;
							if (bVisible)
							{
								MeshVariant->VisibleMeshes.AddUnique(TargetIDToSanitizedName[Prop->TargetID]);
							}
							else
							{
								MeshVariant->HiddenMeshes.AddUnique(TargetIDToSanitizedName[Prop->TargetID]);
							}
						}

						if (PackageVariant && (Prop->PropertyID == EDG_VariantID))
						{
							int SelectedVariantID = -1;
							if (DataNode)
							{
								SelectedVariantID = FCString::Atoi(*DataNode->GetContent());
							}
							PackageVariant->SelectedVariants.Add(SelectedVariantID);
						}

						if (SwitchObjectVariant && (Prop->PropertyID == EDG_ActiveChild))
						{
							int Selection = 0;
							if (DataNode)
							{
								Selection = FCString::Atoi(*DataNode->GetContent());
							}
							SwitchObjectVariant->Selection = Selection;
						}

						if (ObjectSetVariant && DataNode)
						{
							// It seems like being enabled is the default behavior, but let's ignore a value if
							// its explicitly disabled
							if (const FXmlNode* EnabledNode = ValueNode->FindChildNode(TEXT("Enabled")))
							{
								const FString& EnabledValue = EnabledNode->GetContent();
								if (EnabledValue == TEXT("0"))
								{
									continue;
								}
							}

							FDeltaGenVarDataObjectSetVariantValue* NewValue = new (ObjectSetVariant->Values) FDeltaGenVarDataObjectSetVariantValue;
							NewValue->TargetNodeNameSanitized = TargetNodeNameForProperties[PropertyIndex];

							switch(Prop->PropertyID)
							{
							case EDG_Translation:
							{
								float X = FCString::Atof(*DataNode->FindChildNode(TEXT("x"))->GetContent());
								float Y = FCString::Atof(*DataNode->FindChildNode(TEXT("y"))->GetContent());
								float Z = FCString::Atof(*DataNode->FindChildNode(TEXT("z"))->GetContent());
								FVector Location = FVector(X, -Y, Z);

								NewValue->DataType = EObjectSetDataType::Translation;
								NewValue->Data.SetNum(sizeof(FVector));
								FMemory::Memcpy(NewValue->Data.GetData(), &Location, sizeof(FVector));
								break;
							}
							case EDG_Rotation:
							{
								const FXmlNode* Euler = DataNode->FindChildNode(TEXT("Euler"));
								float X = FCString::Atof(*Euler->FindChildNode(TEXT("x"))->GetContent());
								float Y = FCString::Atof(*Euler->FindChildNode(TEXT("y"))->GetContent());
								float Z = FCString::Atof(*Euler->FindChildNode(TEXT("z"))->GetContent());
								// just convert ueler to quaternion orientation - we don't need excess angle values information(360 dergree) that euler has
								// for camera orientation, quat will suffice
								FRotator Rotation = FRotator(FMath::RadiansToDegrees(Y), FMath::RadiansToDegrees(Z), FMath::RadiansToDegrees(X)); // Roll, Pitch and Yaw --> X, Y, Z

								NewValue->DataType = EObjectSetDataType::Rotation;
								NewValue->Data.SetNum(sizeof(FRotator));
								FMemory::Memcpy(NewValue->Data.GetData(), &Rotation, sizeof(FRotator));
								break;
							}
							case EDG_Scaling:
							{
								float X = FCString::Atof(*DataNode->FindChildNode(TEXT("x"))->GetContent());
								float Y = FCString::Atof(*DataNode->FindChildNode(TEXT("y"))->GetContent());
								float Z = FCString::Atof(*DataNode->FindChildNode(TEXT("z"))->GetContent());
								FVector Scaling = FVector(X, Y, Z);

								NewValue->DataType = EObjectSetDataType::Scaling;
								NewValue->Data.SetNum(sizeof(FVector));
								FMemory::Memcpy(NewValue->Data.GetData(), &Scaling, sizeof(FVector));
								break;
							}
							case EDG_Visibility:
							{
								const FString& DataStr = DataNode->GetContent();
								bool bVisible = FCString::Atoi(*DataStr) != 0;

								NewValue->DataType = EObjectSetDataType::Visibility;
								NewValue->Data.SetNum(sizeof(bool));
								FMemory::Memcpy(NewValue->Data.GetData(), &bVisible, sizeof(bool));
								break;
							}
							case EDG_Center:
							{
								float X = FCString::Atof(*DataNode->FindChildNode(TEXT("x"))->GetContent());
								float Y = FCString::Atof(*DataNode->FindChildNode(TEXT("y"))->GetContent());
								float Z = FCString::Atof(*DataNode->FindChildNode(TEXT("z"))->GetContent());
								FVector Center = FVector(X, -Y, Z);

								NewValue->DataType = EObjectSetDataType::Center;
								NewValue->Data.SetNum(sizeof(FVector));
								FMemory::Memcpy(NewValue->Data.GetData(), &Center, sizeof(FVector));
								break;
							}
							default:
								break;
							}
						}
					}
				}

				// GEOMETRY_VARIANT uses Targets node to identify which objects are affected with it
				if (const FXmlNode* VariantTargetsNode = VariantNode->FindChildNode(TEXT("Targets")))
				{
					TSet<FString> VisibleMeshes;

					for (const FXmlNode* TargetNode: FXmlNodeChildren(VariantTargetsNode))
					{
						if (TargetNode->GetTag() != TEXT("Target"))
						{
							continue;
						}

						const FXmlNode* TargetNameNode = TargetNode->FindChildNode(TEXT("Name"));
						if (!TargetNameNode)
						{
							continue;
						}

						const FString& Name = TargetNameNode->GetContent();

						if (MeshVariant)
						{
							VisibleMeshes.Add(Name);
						}
					}

					if (MeshVariant)
					{
						for(auto NameID: TargetNameToID)
						{
							if (VisibleMeshes.Contains(NameID.Key))
							{
								MeshVariant->VisibleMeshes.Add(TargetIDToSanitizedName[NameID.Value]);
							}
							else
							{
								MeshVariant->HiddenMeshes.Add(TargetIDToSanitizedName[NameID.Value]);
							}
						}
					}
				}
			}
		}

		return true;
	}

	bool LoadPOSFile(const TCHAR* InFilePath, FDatasmithDeltaGenImportPosResult& OutResult)
	{
		FXmlFile PosFile;

		if (!PosFile.LoadFile(InFilePath))
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Error, "Couldn't open POS file: %ls", *PosFile.GetLastError());
			return false;
		}

		const FXmlNode* RootNode = PosFile.GetRootNode();
		if ("stateMachine" != RootNode->GetTag())
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Error, "Expected stateMachine root node.");
			return false;
		}
		const FXmlNode* StateMachineNode = RootNode;

		for(const FXmlNode* StateNode: FXmlNodeChildren(StateMachineNode))
		{
			if ("stateEngine" != StateNode->GetTag())
			{
				continue;
			}

			FDeltaGenPosDataState* State = new(OutResult.PosStates) FDeltaGenPosDataState;

			for(const FXmlNode* ActionListNode: FXmlNodeChildren(StateNode))
			{
				if ("actionList" != ActionListNode->GetTag())
				{
					continue;
				}

				State->Name = ActionListNode->GetAttribute("name");

				TMap<FString, bool>& ActionListStates = State->States;
				TMap<FName, int32>& ActionListGeometries = State->Switches;
				TMap<FString, FString>& ActionListMaterials = State->Materials;

				for(const FXmlNode* ActionNode: FXmlNodeChildren(ActionListNode))
				{
					if ("action" != ActionNode->GetTag())
					{
						continue;
					}

					auto ActionType = ActionNode->GetAttribute("type");

					const FXmlNode* ActorNode = ActionNode->FindChildNode("actor");
					if (!ActorNode)
					{
						continue;
					}

					const FXmlNode* ValueNode = ActionNode->FindChildNode("value");
					if (!ValueNode)
					{
						continue;
					}

					if ("stateObject" == ActionType)
					{
						ActionListStates.Add(ActorNode->GetContent(), ValueNode->GetContent() == "on");
					}
					else if ("appearance" == ActionType)
					{
						ActionListMaterials.Add(ActorNode->GetContent(), ValueNode->GetContent());
					}
					else if ("switch" == ActionType)
					{
						ActionListGeometries.Add(FName(*ActorNode->GetContent()), FCString::Atoi(*ValueNode->GetContent()));
					}
					else
					{
						UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Unexpected action type %ls.", *ActionType);
					}
				}
			}
		}
		return true;
	}

	void ParsePointsText(const FString& ValuesNodeContent, TArray<FVector4>& OutValues)
	{
		TArray<FString> ValuesStrings;

		// We don't care about animations that only have 1 keyframe
		if (ValuesNodeContent.ParseIntoArray(ValuesStrings, TEXT(";")) < 2)
		{
			return;
		}

		OutValues.Reserve(ValuesStrings.Num());
		for(const FString& ValueString: ValuesStrings)
		{
			TArray<FString> ValueTokens;
			ValueString.ParseIntoArrayWS(ValueTokens);

			FVector4 ValueResult(0, 0, 0, 0);
			for (int32 Index = 0; Index < FMath::Min(ValueTokens.Num(), 4); ++Index)
			{
				ValueResult[Index] = FCString::Atof(*ValueTokens[Index]);
			}

			OutValues.Add(ValueResult);
		}
	}

	void ParseInterpolationCurveNode(const FXmlNode* InInterpolationCurveNode, TArray<FVector4>& OutValues)
	{
		const FXmlNode* ControlVerticesNode = InInterpolationCurveNode->FindChildNode(TEXT("ControlVertices"));
		if (!ControlVerticesNode)
		{
			return;
		}

		const FXmlNode* PositionsNode = ControlVerticesNode->FindChildNode(TEXT("Positions"));
		if (!PositionsNode)
		{
			return;
		}

		const FString& PositionsText = PositionsNode->GetContent();
		ParsePointsText(PositionsText, OutValues);
	}

	// Calculates the intersection between a 2D infinite line that goes through points A and B, and a
	// horizontal line that crosses P
	FVector4 LineHorizontalIntersection(const FVector4& LineA, const FVector4& LineB, const FVector4& P)
	{
		FVector4 R(0, 0, 0, 0);
		if (FMath::IsNearlyEqual(LineA.Y, LineB.Y))
		{
			R = P;
		}
		else if (FMath::IsNearlyEqual(LineA.X, LineB.X))
		{
			R.X = LineA.X;
			R.Y = P.Y;
		}
		else
		{
			R.Y = P.Y;

			// Line: Y = M*X + C
			// R.x = (R.y - C) / M = (P.y - C) / M <--- division ok, as we know LineA.Y != LineB.Y
			// C = LineA.y - M*LineA.x;
			// Subst: R.x = (P.y - LineA.y)/M + LineA.x
			float M = (LineB.Y - LineA.Y) / (LineB.X - LineA.X);
			R.X = (P.Y - LineA.Y) / M + LineA.X;
		}
		return R;
	}

	// Converts control points from a DeltaGen "EaseIn" interpolation curve into a regular cubic curve
	bool ConvertEaseInToCubic(TArray<FVector4>& Values)
	{
		// On this interpolation mode DeltaGen forces you to have exactly 4 control points
		if (Values.Num() != 4)
		{
			return false;
		}

		// Values will look something like this:
		//		P0: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P4: 0.38400000 0.23999999 0.00000000 0.00000000;
		//		P5: 0.95999998 0.95999998 0.00000000 0.00000000;
		//		P6: 0.95999998 0.95999998 0.00000000 0.00000000

		// There is no documentation for what precise formula DeltaGen uses for the
		// EaseIn and out, so this is an approximation, but we will add control points
		// so it ends up looking like this:
		//		P0: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P1: <intersection between the P4-P5 line and the horizontal line that goes through P0>
		//		P2: <midpoint between P1 and P4>
		//		P3: <copy of P4>
		//		P4: 0.38400000 0.23999999 0.00000000 0.00000000;
		//		P5: 0.95999998 0.95999998 0.00000000 0.00000000;
		//		P6: 0.95999998 0.95999998 0.00000000 0.00000000;

		const FVector4& P0 = Values[0];
		const FVector4& P4 = Values[1];
		const FVector4& P5 = Values[2];
		FVector4 P1 = LineHorizontalIntersection(P4, P5, P0);

		Values.Insert({P1, (P1 + P4) * 0.5f, P4}, 1);
		return true;
	}

	// Converts control points from a DeltaGen "EaseOut" interpolation curve into a regular cubic curve
	bool ConvertEaseOutToCubic(TArray<FVector4>& Values)
	{
		// On this interpolation mode DeltaGen forces you to have exactly 4 control points
		if (Values.Num() != 4)
		{
			return false;
		}

		// Values will look something like this:
		//		P0: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P1: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P2: 0.57599998 0.72000003 0.00000000 0.00000000;
		//		P6: 0.95999998 0.95999998 0.00000000 0.00000000;

		// There is no documentation for what precise formula DeltaGen uses for the
		// EaseIn and out, so this is an approximation, but we will add control points
		// so it ends up looking like this:
		//		P0: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P1: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P2: 0.57599998 0.72000003 0.00000000 0.00000000;
		//		P3: <copy of P2>
		//		P4: <midpoint between P2 and P5>
		//		P5: <intersection between the P0-P2 line and the horizontal line that goes through P6>
		//		P6: 0.95999998 0.95999998 0.00000000 0.00000000;

		const FVector4& P0 = Values[0];
		const FVector4& P2 = Values[2];
		const FVector4& P6 = Values[3];
		FVector4 P5 = LineHorizontalIntersection(P0, P2, P6);

		Values.Insert({P2, (P2 + P5) * 0.5f, P5}, 3);
		return true;
	}

	// Converts control points from a DeltaGen "EaseInAndEaseOut" interpolation curve into a regular cubic curve
	bool ConvertEaseInAndEaseOutToCubic(TArray<FVector4>& Values)
	{
		// On this interpolation mode DeltaGen forces you to have exactly 4 control points
		if (Values.Num() != 4)
		{
			return false;
		}

		// Values will look something like this:
		//		P0: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P4: 0.38400000 0.31999999 0.00000000 0.00000000;
		//		P5: 0.57599998 0.63999999 0.00000000 0.00000000;
		//		P9: 0.95999998 0.95999998 0.00000000 0.00000000;

		// There is no documentation for what precise formula DeltaGen uses for the
		// EaseIn and out, so this is an approximation, but we will add control points
		// so it ends up looking like this:
		//		P0: 0.00000000 0.00000000 0.00000000 0.00000000;
		//		P1: <intersection between the P4-P5 line and the horizontal line that goes through P0>
		//		P2: <midpoint between P1 and P4>
		//		P3: <copy of P4>
		//		P5: 0.57599998 0.63999999 0.00000000 0.00000000;
		//		P6: <copy of P5>
		//		P7: <midpoint between P5 and P8>
		//		P8: <intersection between the P4-P5 line and the horizontal line that goes through P9>
		//		P9: 0.95999998 0.95999998 0.00000000 0.00000000;

		const FVector4& P0 = Values[0];
		const FVector4& P4 = Values[1];
		const FVector4& P5 = Values[2];
		const FVector4& P9 = Values[3];

		FVector4 P1 = LineHorizontalIntersection(P4, P5, P0);
		FVector4 P8 = LineHorizontalIntersection(P4, P5, P9);

		Values.Reserve(10);
		Values.Insert({P1, (P1 + P4) * 0.5f, P4}, 1);
		Values.Insert({P5, (P5 + P8) * 0.5f, P8}, 6);

		return true;
	}

	bool ParseInterpolatorNode(const FXmlNode* InInterpolatorNode, EDeltaGenAnimationInterpolation& OutInterpolation, TArray<FVector4>& OutValues)
	{
		TArray<const FXmlNode*> InterpolationCurveNodes;
		for (const FXmlNode* InterpolationCurveNode : FXmlNodeChildren(InInterpolatorNode))
		{
			if (TEXT("InterpolationCurve") == InterpolationCurveNode->GetTag())
			{
				InterpolationCurveNodes.Add(InterpolationCurveNode);
			}
		}

		// It will store either 1 interpolation curve with XYZ values,
		// or 3 interpolation curves with one dimension at a time
		int32 InterpolationCurvesCount = InterpolationCurveNodes.Num();
		if (InterpolationCurvesCount == 1)
		{
			ParseInterpolationCurveNode(InterpolationCurveNodes[0], OutValues);
		}
		else if (InterpolationCurvesCount == 3)
		{
			TArray<FVector4> XValues;
			TArray<FVector4> YValues;
			TArray<FVector4> ZValues;
			ParseInterpolationCurveNode(InterpolationCurveNodes[0], XValues);
			ParseInterpolationCurveNode(InterpolationCurveNodes[1], YValues);
			ParseInterpolationCurveNode(InterpolationCurveNodes[2], ZValues);

			int32 NumValues = XValues.Num();
			ensure(NumValues == YValues.Num() && NumValues == ZValues.Num());

			OutValues.SetNumUninitialized(NumValues);
			for (int32 Index = 0; Index < NumValues; ++Index)
			{
				// It seems to store time in the X component, and the value in the Y component
				OutValues[Index] = FVector4(XValues[Index].Y,
											YValues[Index].Y,
											ZValues[Index].Y,
											0.0f);
			}
		}
		else
		{
			// This should not be possible
			return false;
		}

		bool bSuccess = true;

		FString TypeAttribute = InInterpolatorNode->GetAttribute(TEXT("Type"));
		if (TEXT("Linear") == TypeAttribute)
		{
			OutInterpolation = EDeltaGenAnimationInterpolation::Linear;
		}
		else if(TEXT("Smooth") == TypeAttribute)
		{
			OutInterpolation = EDeltaGenAnimationInterpolation::Cubic;
		}
		else if(TEXT("Constant") == TypeAttribute)
		{
			OutInterpolation = EDeltaGenAnimationInterpolation::Constant;
		}
		else if(TEXT("EaseIn") == TypeAttribute)
		{
			OutInterpolation = EDeltaGenAnimationInterpolation::Cubic;
			bSuccess = ConvertEaseInToCubic(OutValues);
		}
		else if(TEXT("EaseOut") == TypeAttribute)
		{
			OutInterpolation = EDeltaGenAnimationInterpolation::Cubic;
			bSuccess = ConvertEaseOutToCubic(OutValues);
		}
		else if(TEXT("EaseInAndEaseOut") == TypeAttribute)
		{
			OutInterpolation = EDeltaGenAnimationInterpolation::Cubic;
			bSuccess = ConvertEaseInAndEaseOutToCubic(OutValues);
		}
		else
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Unsupported DeltaGen interpolation type '%ls'", *TypeAttribute);
			OutInterpolation = EDeltaGenAnimationInterpolation::Unsupported;
			bSuccess = false;
		}

		return bSuccess;
	}

	// Pulls the float multiplier that is applied to all control points of a TimeAdjustment Interpolator,
	// also normalizing them.
	// Example: [[0, 0], [2, 1], [12, 6], [24, 12]] becomes [[0, 0], [1, 1], [6, 6], [12, 12]], and 2 is returned
	float ExtractEncodedFramerateMultiplier(TArray<FVector4>& ControlPoints)
	{
		float Multiplier = 1.0f;

		if (ControlPoints.Num() > 0)
		{
			const FVector4& LastPoint = ControlPoints.Last();
			if (!FMath::IsNearlyZero(LastPoint.Y))
			{
				Multiplier = LastPoint.X / LastPoint.Y;

				for (FVector4& ControlPoint : ControlPoints)
				{
					ControlPoint.X /= Multiplier;
				}
			}
		}

		return Multiplier;
	}

	bool LoadTMLFile(const TCHAR* InFilePath, FDatasmithDeltaGenImportTmlResult& OutResult)
	{
		FXmlFile TmlFile;

		TArray<FString> FileContentLines;
		if(!FFileHelper::LoadFileToStringArray(FileContentLines, InFilePath))
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Error, "Couldn't read VAR file: %ls", InFilePath);
		}

		FixDeltaGenXml(FileContentLines);

		// it would have been nice to have FXmlFile to expose LoadFile for lines(it splits into lines anyway inside)
		FString FileContent = FString::Join(FileContentLines, TEXT("\n"));

		if (!TmlFile.LoadFile(FileContent, EConstructMethod::ConstructFromBuffer))
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Error, "Couldn't open TML file: %ls", *TmlFile.GetLastError());
			return false;
		}

		const FXmlNode* RootNode = TmlFile.GetRootNode();
		const FXmlNode* ProductAspectsNode = RootNode->FindChildNode(TEXT("ProductAspects"));
		if (!ProductAspectsNode)
		{
			return false;
		}

		const FXmlNode* AspectContainerNode = ProductAspectsNode->FindChildNode(TEXT("AspectContainer"));
		if (!AspectContainerNode)
		{
			return false;
		}

		const FXmlNode* AnimationsNode = nullptr;
		for(const FXmlNode* Node: FXmlNodeChildren(AspectContainerNode))
		{
			if (TEXT("Aspect") != Node->GetTag())
			{
				continue;
			}
			if (TEXT("Animations") != Node->GetAttribute(TEXT("Type")))
			{
				continue;
			}
			AnimationsNode = Node;
			break;
		}

		if (!AnimationsNode)
		{
			return false;
		}

		// Default framerate
		float FileFramerate = 30.0f;

		for(const FXmlNode* Node: FXmlNodeChildren(AnimationsNode))
		{
			if (TEXT("FPS") == Node->GetTag())
			{
				float Framerate = FCString::Atof(*Node->GetContent());
				if (!FMath::IsNearlyZero(Framerate))
				{
					FileFramerate = Framerate;
				}
			}

			if (TEXT("Timeline") != Node->GetTag())
			{
				continue;
			}

			auto AnimationGroupNode = Node;

			FString TimelineName = AnimationGroupNode->GetAttribute(TEXT("Name"));

			FDeltaGenTmlDataTimeline& Timeline = *(new(OutResult.Timelines) FDeltaGenTmlDataTimeline);

			Timeline.Name = TimelineName;

			// DeltaGen encodes the final framerate as a FileFramerate * Multiplier. Strangely enough, the
			// multiplier is encoded into each animation. With this object we keep track of all the multipliers we
			// pulled from the animations of this timeline and check if they're consistent later
			TArray<float> FramerateMultipliers;

			for(const FXmlNode* AnimationNode: FXmlNodeChildren(AnimationGroupNode))
			{
				if (TEXT("Animation") != AnimationNode->GetTag())
				{
					continue;
				}

				// SceneObjectAnimationContainer
				if (TEXT("SceneObjectAxisAngleAnimationContainer") != AnimationNode->GetAttribute(TEXT("Type")) &&
					TEXT("SceneObjectAnimationContainer") != AnimationNode->GetAttribute(TEXT("Type")))
				{
					continue;
				}

				FDeltaGenTmlDataTimelineAnimation& TimelineAnimation = *(new(Timeline.Animations) FDeltaGenTmlDataTimelineAnimation);

				// NormalAnimation, CameraOrientationAnimation, CameraPositionAnimation
				const FXmlNode* SceneObjectAnimationContainerNode = AnimationNode;

				const FXmlNode* TargetResolverNode = SceneObjectAnimationContainerNode->FindChildNode(TEXT("TargetResolver"));

				if (!TargetResolverNode)
				{
					continue;
				}

				TimelineAnimation.DelayMs = FCString::Atof(*SceneObjectAnimationContainerNode->GetAttribute(TEXT("Delay")));

				FString TargetName;
				{
					if (const FXmlNode* TargetNode = TargetResolverNode->FindChildNode(TEXT("Target")))
					{
						if (const FXmlNode* DatasNode = TargetNode->FindChildNode(TEXT("datas")))
						{
							if (const FXmlNode* DataNode = DatasNode->FindChildNode(TEXT("data")))
							{
								if (const FXmlNode* ValueNode = DataNode->FindChildNode(TEXT("value")))
								{
									if (const FXmlNode* MetaDataQueryNode = ValueNode->FindChildNode(TEXT("metaDataQuery")))
									{
										const FXmlNode* ConstraintNode = MetaDataQueryNode;

										// try go go deeper if constraintGroup node is present
										if (const FXmlNode* ConstraintGroupNode = ConstraintNode->FindChildNode(TEXT("constraintGroup")))
										{
											ConstraintNode = ConstraintGroupNode;
										}

										if (const FXmlNode* ObjectNameConstraintNode = ConstraintNode->FindChildNode(TEXT("objectNameConstraint")))
										{
											TargetName = ObjectNameConstraintNode->GetAttribute(TEXT("value"));
										}
									}
								}
							}
						}

						if (TargetName.IsEmpty())
						{
							if (const FXmlNode* NameNode = TargetNode->FindChildNode(TEXT("name")))
							{
								TargetName = NameNode->GetContent();
							}
						}
					}
				}

				TimelineAnimation.TargetNode = FName(*DatasmitDeltaGenSanitizeObjectName(TargetName));

				for(const FXmlNode* ObjectAnimationNode: FXmlNodeChildren(SceneObjectAnimationContainerNode))
				{
					if (TEXT("Animation") != ObjectAnimationNode->GetTag())
					{
						continue;
					}

					EDeltaGenTmlDataAnimationTrackType AnimationType = EDeltaGenTmlDataAnimationTrackType::Unsupported;
					if (TEXT("SceneObjectTranslationAnimation") == ObjectAnimationNode->GetAttribute(TEXT("Type")))
					{
						AnimationType = EDeltaGenTmlDataAnimationTrackType::Translation;
					}
					else if (TEXT("SceneObjectRotationAnimation") == ObjectAnimationNode->GetAttribute(TEXT("Type"))) // quaternion
					{
						AnimationType = EDeltaGenTmlDataAnimationTrackType::Rotation;
					}
					else if (TEXT("SceneObjectEulerAnimation") == ObjectAnimationNode->GetAttribute(TEXT("Type")))  // degrees
					{
						AnimationType = EDeltaGenTmlDataAnimationTrackType::RotationDeltaGenEuler;
					}
					else if (TEXT("SceneObjectScaleAnimation") == ObjectAnimationNode->GetAttribute(TEXT("Type")))
					{
						AnimationType = EDeltaGenTmlDataAnimationTrackType::Scale;
					}
					else if (TEXT("SceneObjectCenterAnimation") == ObjectAnimationNode->GetAttribute(TEXT("Type")))
					{
						AnimationType = EDeltaGenTmlDataAnimationTrackType::Center;
					}

					FDeltaGenTmlDataAnimationTrack& AnimationTrack = *(new(TimelineAnimation.Tracks) FDeltaGenTmlDataAnimationTrack);
					AnimationTrack.Type = AnimationType;
					AnimationTrack.DelayMs = FCString::Atof(*ObjectAnimationNode->GetAttribute(TEXT("Delay")));

					TArray<FVector4> Values;
					TArray<FVector4> ValueControlPoints;
					TArray<FVector4> KeyControlPoints;

					bool bSuccess = true;

					const FXmlNode* AnimationFunctionNodes = ObjectAnimationNode->FindChildNode(TEXT("AnimationFunction"));
					for (const FXmlNode* AnimationFunctionNode: FXmlNodeChildren(AnimationFunctionNodes))
					{
						if (TEXT("Interpolator") == AnimationFunctionNode->GetTag())
						{
							// Parse value control points
							if (TEXT("Base") == AnimationFunctionNode->GetAttribute(TEXT("Role")))
							{
								bSuccess &= ParseInterpolatorNode(AnimationFunctionNode, AnimationTrack.ValueInterpolation, ValueControlPoints);
							}
							// Parse key control points
							if (TEXT("TimeAdjustment") == AnimationFunctionNode->GetAttribute(TEXT("Role")))
							{
								bSuccess &= ParseInterpolatorNode(AnimationFunctionNode, AnimationTrack.KeyInterpolation, KeyControlPoints);
								if (bSuccess)
								{
									FramerateMultipliers.Add(ExtractEncodedFramerateMultiplier(KeyControlPoints));
								}
							}
						}
						else if (TEXT("Sequence") == AnimationFunctionNode->GetTag())
						{
							if (TEXT("Base") == AnimationFunctionNode->GetAttribute(TEXT("Role")))
							{
								const FXmlNode* KeyframesNode = AnimationFunctionNode->FindChildNode(TEXT("Keyframes"));
								if (!KeyframesNode)
								{
									continue;
								}

								const FXmlNode* KeysNode = KeyframesNode->FindChildNode(TEXT("Keys"));
								if (!KeysNode)
								{
									continue;
								}

								const FXmlNode* ValuesNode = KeyframesNode->FindChildNode(TEXT("Values"));
								if (!ValuesNode)
								{
									continue;
								}

								// Parse keys from base track
								const FString& KeysText = KeysNode->GetContent();
								TArray<FString> KeysStrings;
								AnimationTrack.Keys.Empty();
								if (1 < KeysText.ParseIntoArray(KeysStrings, TEXT(";")))
								{
									for(const FString& KeyString: KeysStrings)
									{
										AnimationTrack.Keys.Add(FCString::Atof(*KeyString));
									}
								}

								// Parse values from base track
								const FString& ValuesText = ValuesNode->GetContent();
								ParsePointsText(ValuesText, Values);
							}
						}
					}

					if (!bSuccess)
					{
						UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Invalid animation for node '%ls' within timeline '%ls'!", *TargetName, *TimelineName);
					}

					// We export euler angles as is - because converting to quat loses information
					// E.g. 0 vs. 360 degree euler rotation is same in quaternion representation. But for animation it makes difference.
					// For example, take animation of [(0, 0, 0), (360, 0, 0)] in euler angles, this is full circle rotation
					// this can't be encoded in quats as quats define orientation
					if (AnimationType == EDeltaGenTmlDataAnimationTrackType::Rotation)
					{
						for (FVector4& Value : Values)
						{
							FVector Euler = FQuat(Value[0], -Value[1], Value[2], -Value[3]).Euler();
							Value = FVector4(Euler, 0.0f);
						}
					}

					// Convert to FVector so the entire rest of the importer doesn't have to
					// carry and process FVector4s
					AnimationTrack.Values.Append(Values);
					AnimationTrack.ValueControlPoints.Append(ValueControlPoints);
					AnimationTrack.KeyControlPoints.Append(KeyControlPoints);
				}
			}

			// This is a strange behavior from DeltaGen: If the animation was originally 25fps and it was later
			// converted to 50fps, the .tml file will still contain '25' for the FPS, but the animations will have
			// a "TimeAdjustment" Interpolator with control vertices that have an extra multiplier in the x dimension.
			// Example: In this case, the Interpolator's ControlVertices xml element would look like this:
			//						<Positions>
			//							0.00000000 0.00000000 0.00000000;
			//							6.66666651 3.33333325 0.00000000;
			//							13.33333302 6.66666651 0.00000000;
			//							20.00000000 10.00000000 0.00000000
			//						</Positions>
			// Notice how x * 2 = y: This means that 2 is our factor, so the animation framerate is 25 * 2 = 50fps
			// I have no idea why this is done, or if it actually works like this every time, but that's all the info
			// we have
			float ConsistentMultiplier = FramerateMultipliers.Num() > 0 ? FramerateMultipliers[0] : 1.0f;
			for (float Multiplier : FramerateMultipliers)
			{
				if (!FMath::IsNearlyEqual(Multiplier, ConsistentMultiplier))
				{
					UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Inconsistent framerate for timeline '%ls'", *TimelineName);
					break;
				}
			}

			Timeline.Framerate = FileFramerate * ConsistentMultiplier;
		}
		return true;
	}
}

FDatasmithDeltaGenImportVariantsResult FDatasmithDeltaGenAuxFiles::ParseVarFile(const FString& InFilePath)
{
	FDatasmithDeltaGenImportVariantsResult Result;
	if (InFilePath.IsEmpty())
	{
		return Result;
	}

	if (FPaths::FileExists(InFilePath) && FPaths::GetExtension(InFilePath, false) == TEXT("var"))
	{
		DeltaGenAuxFiles::LoadVARFile(*InFilePath, Result);

		for (const FDeltaGenVarDataVariantSwitch& Switch : Result.VariantSwitches)
		{
			switch (Switch.Type)
			{
			case EDeltaGenVarDataVariantSwitchType::SwitchObject:
				Result.SwitchObjects.Add(Switch.SwitchObject.TargetSwitchObject);
				break;
			case EDeltaGenVarDataVariantSwitchType::Geometry:
				Result.ToggleObjects.Append(Switch.Geometry.TargetNodes);
				break;
			case EDeltaGenVarDataVariantSwitchType::ObjectSet:
				for (const FDeltaGenVarDataObjectSetVariant& Var : Switch.ObjectSet.Variants)
				{
					for (const FDeltaGenVarDataObjectSetVariantValue& Value : Var.Values)
					{
						Result.ObjectSetObjects.Add(Value.TargetNodeNameSanitized);
					}
					Result.ObjectSetObjects.Remove((FName)NAME_None);
				}
				break;
			default:
				break;
			}
		}

		UE_LOGF(LogDatasmithDeltaGenImport, Verbose, "Imported %d variants/variant sets", Result.VariantSwitches.Num());
	}
	else
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Variants file '%ls' doesn't exist or is not a .var file!", *InFilePath);
	}

	return Result;
}

FDatasmithDeltaGenImportPosResult FDatasmithDeltaGenAuxFiles::ParsePosFile(const FString& InFilePath)
{
	FDatasmithDeltaGenImportPosResult Result;
	if (InFilePath.IsEmpty())
	{
		return Result;
	}

	if (FPaths::FileExists(InFilePath) && FPaths::GetExtension(InFilePath, false) == TEXT("pos"))
	{
		DeltaGenAuxFiles::LoadPOSFile(*InFilePath, Result);

		for (const FDeltaGenPosDataState& State : Result.PosStates)
		{
			for (const auto& StatePair : State.States)
			{
				Result.StateObjects.Add(FName(*StatePair.Key));
			}

			for (const auto& SwitchPair : State.Switches)
			{
				Result.SwitchObjects.Add(SwitchPair.Key);
			}

			for (const auto& MaterialPair : State.Materials)
			{
				Result.SwitchMaterialObjects.Add(FName(*MaterialPair.Key));
			}
		}

		UE_LOGF(LogDatasmithDeltaGenImport, Verbose, "Imported %d POS states", Result.PosStates.Num());
	}
	else
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Warning, "POS file '%ls' doesn't exist or is not a .pos file!", *InFilePath);
	}

	return Result;
}

FDatasmithDeltaGenImportTmlResult FDatasmithDeltaGenAuxFiles::ParseTmlFile(const FString& InFilePath)
{
	FDatasmithDeltaGenImportTmlResult Result;
	if (InFilePath.IsEmpty())
	{
		return Result;
	}

	if (FPaths::FileExists(InFilePath) && FPaths::GetExtension(InFilePath, false) == TEXT("tml"))
	{
		DeltaGenAuxFiles::LoadTMLFile(*InFilePath, Result);

		for (const FDeltaGenTmlDataTimeline& Timeline: Result.Timelines)
		{
			for (const FDeltaGenTmlDataTimelineAnimation& Animation : Timeline.Animations)
			{
				Result.AnimatedObjects.Add(Animation.TargetNode);
			}
		}

		UE_LOGF(LogDatasmithDeltaGenImport, Verbose, "Imported animations of %d nodes", Result.AnimatedObjects.Num());
	}
	else
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Warning, "TML file '%ls' doesn't exist or is not a .tml file!", *InFilePath);
	}

	return Result;
}


