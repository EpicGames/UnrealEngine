// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "UObject/NameTypes.h"

struct FDeltaGenPosDataState;
struct FDeltaGenTmlDataTimeline;

struct FDeltaGenVarDataVariantSwitch;

class FDatasmithDeltaGenImportVariantsResult
{
public:
	TArray<FDeltaGenVarDataVariantSwitch> VariantSwitches;
	TSet<FName> SwitchObjects;
	TSet<FName> ToggleObjects;
	TSet<FName> ObjectSetObjects;
};

class FDatasmithDeltaGenImportTmlResult
{
public:
	TArray<FDeltaGenTmlDataTimeline> Timelines;
	TSet<FName> AnimatedObjects;
};

class FDatasmithDeltaGenImportPosResult
{
public:
	TArray<FDeltaGenPosDataState> PosStates;
	TSet<FName> StateObjects;
	TSet<FName> SwitchObjects;
	TSet<FName> SwitchMaterialObjects;
};

class FDatasmithDeltaGenAuxFiles
{
public:
	static FDatasmithDeltaGenImportVariantsResult ParseVarFile(const FString& InFilePath);
	static FDatasmithDeltaGenImportPosResult ParsePosFile(const FString& InFilePath);
	static FDatasmithDeltaGenImportTmlResult ParseTmlFile(const FString& InFilePath);
};


