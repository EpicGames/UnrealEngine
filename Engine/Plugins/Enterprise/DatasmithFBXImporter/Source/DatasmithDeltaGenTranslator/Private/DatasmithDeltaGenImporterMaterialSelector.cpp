// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenImporterMaterialSelector.h"
#include "IDatasmithSceneElements.h"

#include "UObject/SoftObjectPath.h"

FDatasmithDeltaGenImporterMaterialSelector::FDatasmithDeltaGenImporterMaterialSelector()
{
	// Opaque
	ReferenceMaterial.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/FBXImporter/DeltaGenReference.DeltaGenReference") );

	// Transparent
	ReferenceMaterialTransparent.FromSoftObjectPath( FSoftObjectPath("/DatasmithContent/Materials/FBXImporter/DeltaGenReferenceTransparent.DeltaGenReferenceTransparent") );
}

bool FDatasmithDeltaGenImporterMaterialSelector::IsValid() const
{
	return ReferenceMaterial.IsValid() && ReferenceMaterialTransparent.IsValid();
}

const FDatasmithReferenceMaterial& FDatasmithDeltaGenImporterMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const
{
	// Transparent material
	const TSharedPtr< IDatasmithKeyValueProperty > OpacityProperty = InDatasmithMaterial->GetPropertyByName(TEXT("Opacity"));

	float OpacityValue;

	if ( OpacityProperty.IsValid() &&
		GetFloat( OpacityProperty, OpacityValue ) &&
		OpacityValue < 1.f )
	{
		return ReferenceMaterialTransparent;
	}

	return ReferenceMaterial;
}

bool FDatasmithDeltaGenImporterMaterialSelector::IsValidMaterialType( EDatasmithReferenceMaterialType InType ) const
{
	return InType == EDatasmithReferenceMaterialType::Auto || InType == EDatasmithReferenceMaterialType::Opaque || InType == EDatasmithReferenceMaterialType::Transparent;
}
