// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"


enum class EDatasmithReferenceMaterialType : uint8;

class IDatasmithMaterialInstanceElement;

class FDatasmithDeltaGenImporterMaterialSelector : public FDatasmithReferenceMaterialSelector
{
public:
	FDatasmithDeltaGenImporterMaterialSelector();

	virtual bool IsValid() const override;
	virtual const FDatasmithReferenceMaterial& GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const override;

protected:
	bool IsValidMaterialType( EDatasmithReferenceMaterialType InType ) const;

private:
	FDatasmithReferenceMaterial ReferenceMaterial;
	FDatasmithReferenceMaterial ReferenceMaterialTransparent;
};
