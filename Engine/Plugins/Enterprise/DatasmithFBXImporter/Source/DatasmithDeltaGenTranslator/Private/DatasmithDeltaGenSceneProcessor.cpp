// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenSceneProcessor.h"

#include "DatasmithDeltaGenImportData.h"
#include "DatasmithDeltaGenLog.h"
#include "DatasmithUtils.h"

#include "Factories/TextureFactory.h"
#include "HAL/FileManager.h"

using TimelineToAnimations = TMap<FDeltaGenTmlDataTimeline*, FDeltaGenTmlDataTimelineAnimation*>;

namespace DeltaGenProcessorImpl
{
	// Moves tracks of TrackTypes to new animations within each Timeline, all tied to the Dummy node's name
	void MoveTracksToDummyAnimation(TSharedPtr<FDatasmithFBXSceneNode>& Dummy,
									EDeltaGenTmlDataAnimationTrackType TrackTypes,
									const FVector& TransOffset,
									TimelineToAnimations* FoundAnimations,
									TMap<FDeltaGenTmlDataTimeline *, TArray<FDeltaGenTmlDataTimelineAnimation>> &NewAnimationsPerTimeline)
	{
		for (TPair<FDeltaGenTmlDataTimeline*, FDeltaGenTmlDataTimelineAnimation*>& AnimationInTimeline : *FoundAnimations)
		{
			FDeltaGenTmlDataTimeline* Timeline = AnimationInTimeline.Key;
			FDeltaGenTmlDataTimelineAnimation* Animation = AnimationInTimeline.Value;

			TArray<FDeltaGenTmlDataAnimationTrack>& Tracks = Animation->Tracks;
			TArray<FDeltaGenTmlDataAnimationTrack> NewTracks;
			NewTracks.Reserve(Tracks.Num());

			for (int32 TrackIndex = Tracks.Num() - 1; TrackIndex >= 0; --TrackIndex)
			{
				FDeltaGenTmlDataAnimationTrack& ThisTrack = Tracks[TrackIndex];

				if (EnumHasAnyFlags(ThisTrack.Type, TrackTypes))
				{
					if (EnumHasAnyFlags(ThisTrack.Type, EDeltaGenTmlDataAnimationTrackType::Translation))
					{
						for (FVector& Value : ThisTrack.Values)
						{
							Value += TransOffset;
						}
					}

					NewTracks.Add(ThisTrack);
					Tracks.RemoveAt(TrackIndex);
				}
			}

			// Move tracks to the dummy
			if (NewTracks.Num() > 0)
			{
				TArray<FDeltaGenTmlDataTimelineAnimation>& NewAnimations = NewAnimationsPerTimeline.FindOrAdd(Timeline);

				FDeltaGenTmlDataTimelineAnimation* NewAnimation = new(NewAnimations) FDeltaGenTmlDataTimelineAnimation;
				NewAnimation->TargetNode = *Dummy->Name;
				NewAnimation->Tracks = MoveTemp(NewTracks);

				Dummy->KeepNode();
			}
		}
	}

	void DecomposeRotationPivotsForNode(TSharedPtr<FDatasmithFBXSceneNode> Node,
									    TMap<FString, TimelineToAnimations>& NodeNamesToAnimations,
										TMap<FDeltaGenTmlDataTimeline*, TArray<FDeltaGenTmlDataTimelineAnimation>>& NewAnimationsPerTimeline)
	{
		if (!Node.IsValid() || Node->RotationPivot.IsNearlyZero())
		{
			return;
		}

		TSharedPtr<FDatasmithFBXSceneNode> NodeParent = Node->Parent.Pin();
		if (!NodeParent.IsValid())
		{
			return;
		}

		FVector RotPivot = Node->RotationPivot;
		FVector NodeLocation = Node->LocalTransform.GetTranslation();
		FQuat NodeRotation = Node->LocalTransform.GetRotation();

		Node->RotationPivot.Set(0.0f, 0.0f, 0.0f);
		Node->LocalTransform.SetTranslation(-RotPivot);
		Node->LocalTransform.SetRotation(FQuat::Identity);

		TSharedPtr<FDatasmithFBXSceneNode> Dummy = MakeShared<FDatasmithFBXSceneNode>();
		Dummy->Name = Node->Name + TEXT("_RotationPivot");
		Dummy->OriginalName = Dummy->Name;
		Dummy->SplitNodeID = Node->SplitNodeID;
		Dummy->LocalTransform.SetTranslation(NodeLocation + RotPivot);
		Dummy->LocalTransform.SetRotation(NodeRotation);

		const FVector TransOffset = RotPivot + Node->RotationOffset + Node->ScalingOffset;

		if (TimelineToAnimations* FoundAnimations = NodeNamesToAnimations.Find(Node->OriginalName))
		{
			MoveTracksToDummyAnimation(Dummy,
									   EDeltaGenTmlDataAnimationTrackType::Rotation | EDeltaGenTmlDataAnimationTrackType::RotationDeltaGenEuler | EDeltaGenTmlDataAnimationTrackType::Translation,
									   TransOffset,
									   FoundAnimations,
									   NewAnimationsPerTimeline);
		}

		// Fix hierarchy (place Dummy between Node and Parent)
		Dummy->AddChild(Node);
		NodeParent->Children.Remove(Node);
		NodeParent->AddChild(Dummy);
	}

	void DecomposeScalingPivotsForNode(TSharedPtr<FDatasmithFBXSceneNode> Node,
									   TMap<FString, TimelineToAnimations>& NodeNamesToAnimations,
									   TMap<FDeltaGenTmlDataTimeline*, TArray<FDeltaGenTmlDataTimelineAnimation>>& NewAnimationsPerTimeline)
	{
		if (!Node.IsValid() || Node->ScalingPivot.IsNearlyZero())
		{
			return;
		}

		TSharedPtr<FDatasmithFBXSceneNode> NodeParent = Node->Parent.Pin();
		if (!NodeParent.IsValid())
		{
			return;
		}

		FVector ScalingPivot = Node->ScalingPivot;
		FVector NodeLocation = Node->LocalTransform.GetTranslation();
		FVector NodeScaling = Node->LocalTransform.GetScale3D();

		Node->ScalingPivot.Set(0.0f, 0.0f, 0.0f);
		Node->LocalTransform.SetTranslation(-ScalingPivot);
		Node->LocalTransform.SetScale3D(FVector::OneVector);

		TSharedPtr<FDatasmithFBXSceneNode> Dummy = MakeShared<FDatasmithFBXSceneNode>();
		Dummy->Name = Node->Name + TEXT("_ScalingPivot");
		Dummy->OriginalName = Dummy->Name;
		Dummy->SplitNodeID = Node->SplitNodeID;
		Dummy->LocalTransform.SetTranslation(NodeLocation + ScalingPivot);
		Dummy->LocalTransform.SetScale3D(NodeScaling);

		const FVector TransOffset = ScalingPivot + Node->RotationOffset + Node->ScalingOffset;

		if (TimelineToAnimations* FoundAnimations = NodeNamesToAnimations.Find(Node->OriginalName))
		{
			MoveTracksToDummyAnimation(Dummy,
				EDeltaGenTmlDataAnimationTrackType::Scale | EDeltaGenTmlDataAnimationTrackType::Translation,
				TransOffset,
				FoundAnimations,
				NewAnimationsPerTimeline);
		}

		// Fix hierarchy (place Dummy between Node and Parent)
		Dummy->AddChild(Node);
		NodeParent->Children.Remove(Node);
		NodeParent->AddChild(Dummy);
	}

	/** Will return a clone of Original that has TexAO set with 'AOPath' as its path. It may reuse an existing clone, or create a new one */
	TSharedPtr<FDatasmithFBXSceneMaterial> CloneMaterialForAOTexture(const TSharedPtr<FDatasmithFBXSceneMaterial>& Original, const FString& AOPath)
	{
		if (!Original.IsValid() || AOPath.IsEmpty())
		{
			return nullptr;
		}

		// Maybe we already cloned this material for that AO texture?
		for (const TWeakPtr<FDatasmithFBXSceneMaterial>& Clone : Original->ClonedMaterials)
		{
			if (!Clone.IsValid())
			{
				continue;
			}

			TSharedPtr<FDatasmithFBXSceneMaterial> PinnedClone = Clone.Pin();
			if (FDatasmithFBXSceneMaterial::FTextureParams* AOTexture = PinnedClone->TextureParams.Find(TEXT("TexAO")))
			{
				if (AOTexture->Path == AOPath)
				{
					return PinnedClone;
				}
			}
		}

		TSharedPtr<FDatasmithFBXSceneMaterial> Clone = MakeShared<FDatasmithFBXSceneMaterial>(*Original);
		Clone->TextureParams.FindOrAdd(TEXT("TexAO")).Path = AOPath;

		Original->ClonedMaterials.Add(Clone);

		return Clone;
	}

	/**
	 * Will look for an existing AO texture for a given mesh name and set it on the material.
	 * If it can't set the texture (already has a different AO texture), it will return a cloned version of the material with the new AO texture.
	 */
	TSharedPtr<FDatasmithFBXSceneMaterial> FetchAOTexture(const FString& MeshName, TSharedPtr<FDatasmithFBXSceneMaterial>& Material, const TArray<FDirectoryPath>& TextureFolders)
	{
		UTextureFactory* TextureFactory = UTextureFactory::StaticClass()->GetDefaultObject<UTextureFactory>();
		if (TextureFolders.Num() == 0 || !TextureFactory)
		{
			return nullptr;
		}

		// Find all filepaths for images that are in a texture folder and have the mesh name as part of the filename
		TArray<FString> PotentialTextures;
		for (const FDirectoryPath& Dir : TextureFolders)
		{
			TArray<FString> Textures;
			IFileManager::Get().FindFiles(Textures, *Dir.Path, TEXT(""));

			for (const FString& Texture : Textures)
			{
				if (TextureFactory->FactoryCanImport(Texture) && Texture.Contains(MeshName))
				{
					PotentialTextures.Add(Dir.Path / Texture);
				}
			}
		}

		if (PotentialTextures.Num() == 0)
		{
			return nullptr;
		}

		FString AOTexPath = PotentialTextures[0];
		if (PotentialTextures.Num() > 1)
		{
			UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Found more than one candidate for an AO texture matching mesh name '%ls', so texture '%ls' will be used. Better matching between texture and mesh name would help.", *MeshName, *AOTexPath);
		}

		TSharedPtr<FDatasmithFBXSceneMaterial> ClonedMaterial = nullptr;
		if (FDatasmithFBXSceneMaterial::FTextureParams* ExistingAOTexture = Material->TextureParams.Find(TEXT("TexAO")))
		{
			// If the material already has a different AO texture set, some other mesh is using it, so we need
			// to duplicate this material and set our own AO texture
			if (ExistingAOTexture->Path != AOTexPath)
			{
				ClonedMaterial = DeltaGenProcessorImpl::CloneMaterialForAOTexture(Material, AOTexPath);
			}
		}
		// No AO texture on this material yet, lets set our texture to it and use it
		else
		{
			FDatasmithFBXSceneMaterial::FTextureParams& Tex = Material->TextureParams.FindOrAdd(TEXT("TexAO"));
			Tex.Path = AOTexPath;
		}

		return ClonedMaterial;
	}
};

FDatasmithDeltaGenSceneProcessor::FDatasmithDeltaGenSceneProcessor(FDatasmithFBXScene* InScene)
	: FDatasmithFBXSceneProcessor(InScene)
{
}

void FDatasmithDeltaGenSceneProcessor::SetupAOTextures(const TArray<FDirectoryPath>& TextureFolders)
{
	TSet<TSharedPtr<FDatasmithFBXSceneMaterial>> UniqueMaterials{ Scene->Materials };

	FDatasmithUniqueNameProvider UniqueNameProvider;
	for (const TSharedPtr<FDatasmithFBXSceneMaterial>& UniqueMaterial : UniqueMaterials)
	{
		UniqueNameProvider.AddExistingName(UniqueMaterial->Name);
	}

	for (TSharedPtr<FDatasmithFBXSceneNode>& Node : Scene->GetAllNodes())
	{
		for (int32 Index = 0; Index < Node->Materials.Num(); ++Index)
		{
			TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[Index];

			TSharedPtr<FDatasmithFBXSceneMaterial> ClonedMaterial = DeltaGenProcessorImpl::FetchAOTexture(Node->Mesh->Name, Material, TextureFolders);
			if (ClonedMaterial.IsValid())
			{
				ClonedMaterial->Name = UniqueNameProvider.GenerateUniqueName(ClonedMaterial->Name);

				UniqueMaterials.Add(ClonedMaterial);
				Node->Materials[Index] = ClonedMaterial;
			}
		}
	}

	Scene->Materials = UniqueMaterials.Array();
}

void FDatasmithDeltaGenSceneProcessor::DecomposePivots(TArray<FDeltaGenTmlDataTimeline>& Timelines)
{
	// Cache node names to all the animations they have on all timelines
	TMap<FString, TimelineToAnimations> NodeNamesToAnimations;
	for (FDeltaGenTmlDataTimeline& Timeline : Timelines)
	{
		for (FDeltaGenTmlDataTimelineAnimation& Animation : Timeline.Animations)
		{
			TimelineToAnimations& Animations = NodeNamesToAnimations.FindOrAdd(Animation.TargetNode.ToString());
			Animations.Add(&Timeline, &Animation);
		}
	}

	// Iterate over this array so that we don't step into any newly generated dummy actors
	TMap<FDeltaGenTmlDataTimeline*, TArray<FDeltaGenTmlDataTimelineAnimation>> NewAnimationsPerTimeline;
	for (TSharedPtr<FDatasmithFBXSceneNode> Node : Scene->GetAllNodes())
	{
		DeltaGenProcessorImpl::DecomposeRotationPivotsForNode(Node, NodeNamesToAnimations, NewAnimationsPerTimeline);
		DeltaGenProcessorImpl::DecomposeScalingPivotsForNode(Node, NodeNamesToAnimations, NewAnimationsPerTimeline);
	}

	// Add the new animations only afterwards, as NodeNamesToAnimations stores raw pointers to animations within TArrays
	// that might reallocate somewhere else
	for (TTuple<FDeltaGenTmlDataTimeline*, TArray<FDeltaGenTmlDataTimelineAnimation>> Pair : NewAnimationsPerTimeline)
	{
		FDeltaGenTmlDataTimeline* Timeline = Pair.Key;
		const TArray<FDeltaGenTmlDataTimelineAnimation>& NewAnimations = Pair.Value;

		Timeline->Animations.Append(NewAnimations);
	}
}

