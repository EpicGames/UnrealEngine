// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFBXSceneProcessor.h"


struct FDatasmithFBXScene;
struct FDeltaGenTmlDataTimeline;
struct FDirectoryPath;

class FDatasmithDeltaGenSceneProcessor : public FDatasmithFBXSceneProcessor
{
public:
	FDatasmithDeltaGenSceneProcessor(FDatasmithFBXScene* InScene);

	/**
	 * Fetches AO textures matching corresponding mesh names, and assigns them to materials used for each node.
	 * May create additional materials, as we may have to clone material instances to use different AO textures per mesh.
	 */
	void SetupAOTextures(const TArray<FDirectoryPath>& TextureFolders);

	/**
	 * Decompose all scene nodes with nonzero rotation and scaling pivots
	 * using dummy actors, and handle their animations
	 */
	void DecomposePivots(TArray<FDeltaGenTmlDataTimeline>& Timelines);
};
