// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenTranslator.h"

#include "DatasmithDeltaGenImportOptions.h"
#include "IDatasmithSceneElements.h"
#include "DatasmithDeltaGenImporter.h"
#include "DatasmithDeltaGenLog.h"
#include "DatasmithPayload.h"
#include "HAL/FileManager.h"

void FDatasmithDeltaGenTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
	OutCapabilities.bIsEnabled = true;
	OutCapabilities.bParallelLoadStaticMeshSupported = true;

	TArray<FFileFormatInfo>& Formats = OutCapabilities.SupportedFileFormats;
    Formats.Emplace(TEXT("fbx"), TEXT("DeltaGen Fbx files"));
}

bool FDatasmithDeltaGenTranslator::IsSourceSupported(const FDatasmithSceneSource& Source)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithDeltaGenTranslator::IsSourceSupported)

	const FString& FilePath = Source.GetSourceFile();
	const FString& Extension = Source.GetSourceFileExtension();
	if (!Extension.Equals(TEXT("fbx"), ESearchCase::IgnoreCase))
	{
		return false;
	}

	FArchive* Reader = IFileManager::Get().CreateFileReader( *FilePath );
	if( !Reader )
	{
		return false;
	}

	ANSICHAR Header[64*1024] = { 0 };
	Reader->Serialize(Header, FMath::Min(Reader->TotalSize(), (int64)sizeof(Header) - 1));
	delete Reader;

	// Replace 0 with anything for Strstr to work on binary files
	for (int32 Index = 0; Index < sizeof(Header) - 1; ++Index)
	{
		if (Header[Index] == '\0')
		{
			Header[Index] = '.';
		}
	}

	// Quick and dirty way of identifying with a high degree of confidence if the file
	// is from DeltaGen without parsing the whole scene using the SDK which can take a
	// long time for big scenes.
	//
	// Supports both ASCII and binary formats.
	{
		const ANSICHAR* TagName = FPlatformString::Strstr(Header, "Original|ApplicationName");
		if (TagName)
		{
			const ANSICHAR* TagType = FPlatformString::Strstr(TagName, "KString");
			if (TagType)
			{
				const ANSICHAR* TagData = FPlatformString::Strstr(TagType, "RTT_AG");
				if (TagData)
				{
					// The whole tag should be in the same vicinity
					return (TagData - TagName) < 256;
				}
			}
		}
	}

	return false;
}

bool FDatasmithDeltaGenTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithDeltaGenTranslator::LoadScene)

	OutScene->SetHost(TEXT("DeltaGenTranslator"));

    Importer = MakeShared<FDatasmithDeltaGenImporter>(OutScene, ImportOptions.Get());

	const FString& FilePath = GetSource().GetSourceFile();
	if(!Importer->OpenFile(FilePath))
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Log, "Failed to open file '%ls'!", *FilePath);
		return false;
	}

	if (!Importer->SendSceneToDatasmith())
	{
		UE_LOGF(LogDatasmithDeltaGenImport, Log, "Failed to convert the DeltaGen FBX scene '%ls' to Datasmith!", OutScene->GetName());
		return false;
	}

	return true;
}

void FDatasmithDeltaGenTranslator::UnloadScene()
{
	if (Importer)
	{
		Importer->UnloadScene();
	}
}

bool FDatasmithDeltaGenTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	if (ensure(Importer.IsValid()))
	{
		TArray<FMeshDescription> MeshDescriptions;
		Importer->GetGeometriesForMeshElementAndRelease(MeshElement, MeshDescriptions);
		if (MeshDescriptions.Num() > 0)
		{
			OutMeshPayload.LodMeshes.Add(MoveTemp(MeshDescriptions[0]));
			return true;
		}
	}

	return false;
}

bool FDatasmithDeltaGenTranslator::LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload)
{
	//if (ensure(Importer.IsValid()))
	//{
	//	const TArray<TSharedRef<IDatasmithLevelSequenceElement>>& ImportedSequences = Importer->GetImportedSequences();
	//	if (ImportedSequences.Contains(LevelSequenceElement))
	//	{
	//		// #ueent_todo: move data to OutLevelSequencePayload
	//		// Right now the LevelSequenceElement is imported out of the IDatasmithScene
	//		return true;
	//	}
	//}

	return false;
}

void FDatasmithDeltaGenTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	if (!ImportOptions.IsValid())
	{
		const FString& FilePath = GetSource().GetSourceFile();

		ImportOptions = Datasmith::MakeOptions<UDatasmithDeltaGenImportOptions>();
		ImportOptions->ResetPaths(FilePath, false);
	}

	Options.Add(ImportOptions.Get());
}

void FDatasmithDeltaGenTranslator::SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	for (const TObjectPtr<UDatasmithOptionsBase>& OptionPtr : Options)
	{
		if (UDatasmithDeltaGenImportOptions* InImportOptions = Cast<UDatasmithDeltaGenImportOptions>(OptionPtr))
		{
			ImportOptions.Reset(InImportOptions);
		}
	}

	if (Importer.IsValid())
	{
		Importer->SetImportOptions(ImportOptions.Get());
	}
}
