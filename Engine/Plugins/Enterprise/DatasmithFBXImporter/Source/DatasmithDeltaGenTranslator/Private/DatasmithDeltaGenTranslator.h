// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithTranslator.h"

class FDatasmithSceneSource;
class UDatasmithDeltaGenImportOptions;

class FDatasmithDeltaGenTranslator : public IDatasmithTranslator
{
public:
	virtual FName GetFName() const override { return "DatasmithDeltaGenTranslator"; };

	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;
	virtual bool IsSourceSupported(const FDatasmithSceneSource& Source);

	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;
	virtual void UnloadScene() override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;
	virtual bool LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload) override;

	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;

private:
    TStrongObjectPtr<UDatasmithDeltaGenImportOptions> ImportOptions;
    TSharedPtr<class FDatasmithDeltaGenImporter> Importer;
};
