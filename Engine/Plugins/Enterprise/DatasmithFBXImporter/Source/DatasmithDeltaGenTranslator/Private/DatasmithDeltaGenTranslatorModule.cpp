// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenTranslatorModule.h"
#include "DatasmithDeltaGenImportOptions.h" // IWYU pragma: keep
#include "DatasmithDeltaGenTranslator.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"
#include "DatasmithDeltaGenImporterMaterialSelector.h"


class FDeltaGenTranslatorModule : public IDatasmithDeltaGenTranslatorModule
{
public:
	virtual void StartupModule() override
	{
		// Make sure the DatasmithImporter module exists and has been initialized before adding FDatasmithDeltaGenTranslator's material selector
		FModuleManager::Get().LoadModule(TEXT("DatasmithTranslator"));

		FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("Deltagen"), MakeShared< FDatasmithDeltaGenImporterMaterialSelector >());

		Datasmith::RegisterTranslator<FDatasmithDeltaGenTranslator>();
	}

	virtual void ShutdownModule() override
	{
		Datasmith::UnregisterTranslator<FDatasmithDeltaGenTranslator>();
	}
};

IMPLEMENT_MODULE(FDeltaGenTranslatorModule, DatasmithDeltaGenTranslator);
