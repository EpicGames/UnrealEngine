// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithDeltaGenVariantConverter.h"

#include "DatasmithDeltaGenImportData.h"
#include "DatasmithDeltaGenLog.h"
#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "DatasmithVariantElements.h"

namespace FDeltaGenVariantConverterImpl
{
	// Adds data from InVariantA into InOutVariantB, combining bindings for the same actor
	// Note that this might lead to some property values/bindings being pointed to by multiple parents, but that
	// is fine: When creating the LevelVariantSets asset it will lead to creating separate variants like usual
	void MergeVariants(TSharedPtr<IDatasmithVariantElement> FromVariantA, TSharedPtr<IDatasmithVariantElement> IntoVariantB)
	{
		TMap<TWeakPtr<IDatasmithActorElement>, TSharedPtr<IDatasmithActorBindingElement>> BindingsInB;
		for (int32 BindingIndex = 0; BindingIndex < IntoVariantB->GetActorBindingsCount(); ++BindingIndex)
		{
			TSharedPtr<IDatasmithActorBindingElement> Binding = IntoVariantB->GetActorBinding(BindingIndex);
			TWeakPtr<IDatasmithActorElement> Actor = Binding->GetActor();
			BindingsInB.Add(Actor, Binding);
		}

		for (int32 BindingIndex = 0; BindingIndex < FromVariantA->GetActorBindingsCount(); ++BindingIndex)
		{
			TSharedPtr<IDatasmithActorBindingElement> BindingFromA = FromVariantA->GetActorBinding(BindingIndex);
			TWeakPtr<IDatasmithActorElement> Actor = BindingFromA->GetActor();

			// There is already a binding for Actor in IntoVariantB --> Move all property captures over
			// It doesn't matter if there are multiple captures to the same property, as only one UPropertyCapture is ever created
			if (TSharedPtr<IDatasmithActorBindingElement>* FoundBindingInB = BindingsInB.Find(Actor))
			{
				// Don't merge a binding into itself
				if (BindingFromA == *FoundBindingInB)
				{
					continue;
				}

				for (int32 PropIndex = 0; PropIndex < BindingFromA->GetPropertyCapturesCount(); ++PropIndex)
				{
					TSharedPtr<IDatasmithBasePropertyCaptureElement> PropertyCapture = BindingFromA->GetPropertyCapture(PropIndex);
					(*FoundBindingInB)->AddPropertyCapture(PropertyCapture.ToSharedRef());
				}
			}
			// There is no binding for Actor in IntoVariantB --> Move the entire binding to IntoVariantB
			else
			{
				IntoVariantB->AddActorBinding(BindingFromA.ToSharedRef());
				BindingsInB.Add(Actor, BindingFromA);
			}
		}
	}
}

TSharedPtr<IDatasmithLevelVariantSetsElement> FDeltaGenVariantConverter::ConvertVariants(TArray<FDeltaGenVarDataVariantSwitch>& Vars, TArray<FDeltaGenPosDataState>& PosStates, FActorMap& ActorsByOriginalName, FMaterialMap& MaterialsByName)
{
	if (Vars.Num() == 0 && PosStates.Num() == 0)
	{
		return nullptr;
	}

	TSharedPtr<IDatasmithLevelVariantSetsElement> LVS = FDatasmithSceneFactory::CreateLevelVariantSets(TEXT("LevelVariantSets"));

	// Remember all vars and varsets we created so that we can reference them later from Package variants
	TMap<TSharedPtr<IDatasmithVariantSetElement>, FString> CreatedVariantSetsToOriginalNames;

	TMap<FString, FDeltaGenVarDataVariantSwitch*> NonPackageVariantsByName;
	TArray<FDeltaGenVarDataVariantSwitch*> PackageVariants;

	// We may have to create some actors to meet some SwitchObject requirements, so we need to make sure
	// their names are unique
	FDatasmithUniqueNameProvider UniqueNameProvider;
	for (TPair<FName, TArray<TSharedPtr<IDatasmithActorElement>>> Pair : ActorsByOriginalName)
	{
		for (const TSharedPtr<IDatasmithActorElement>& Actor : Pair.Value)
		{
			UniqueNameProvider.AddExistingName(Actor->GetName());
		}
	}

	// Remember for which Switch actors we already created an extra actor to serve as the "disabled" option. Without this,
	// if we had multiple switch variants that target the same actor, we would end up with multiple "disabled" actors
	TMap<TSharedPtr<IDatasmithActorElement>, TSharedPtr<IDatasmithActorElement>> CreatedSwitchObjectNullActors;

	// We'll parse them separately because the package variants will refer to the created
	// non-package variants
	for (FDeltaGenVarDataVariantSwitch& Var : Vars)
	{
		if (Var.Type == EDeltaGenVarDataVariantSwitchType::Package)
		{
			PackageVariants.Add(&Var);
		}
		else
		{
			NonPackageVariantsByName.Add(Var.Name, &Var);
		}
	}

	// See if we have a camera actor, to which we'll bind all our camera variants
	TSharedPtr<IDatasmithActorElement> CameraActor = nullptr;
	if (TArray<TSharedPtr<IDatasmithActorElement>>* FoundCameraActors = ActorsByOriginalName.Find(SCENECAMERA_NAME))
	{
		if (FoundCameraActors)
		{
			for (TSharedPtr<IDatasmithActorElement>& FoundCameraActor : *FoundCameraActors)
			{
				const FString CameraCode = TEXT("-1");
				if (FoundCameraActor.IsValid() && FoundCameraActor->GetTagsCount() >= 2 && CameraCode.Equals(FoundCameraActor->GetTag(1)))
				{
					CameraActor = FoundCameraActor;
					break;
				}
			}
		}
	}

	// Create a VM variant set for each non-package variant set
	for (const TPair<FString, FDeltaGenVarDataVariantSwitch*>& Pair : NonPackageVariantsByName)
	{
		FString VarName = Pair.Key;
		FDeltaGenVarDataVariantSwitch* Var = Pair.Value;

		switch (Var->Type)
		{
		case EDeltaGenVarDataVariantSwitchType::Camera:
		{
			TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(*VarName);
			LVS->AddVariantSet(NewVarSet.ToSharedRef());
			CreatedVariantSetsToOriginalNames.Add(NewVarSet, VarName);

			if (CameraActor.IsValid())
			{
				for (const FDeltaGenVarDataCameraVariant& Option : Var->Camera.Variants)
				{
					TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*Option.Name);
					NewVarSet->AddVariant(NewVar.ToSharedRef());

					FVector Location = Option.Location;
					FRotator Rotation = (Option.Rotation.Quaternion() * FRotator(-90.0f, 0, -90.0f).Quaternion()).Rotator();
					FVector Scale = FVector(1.0f, 1.0f, 1.0f);

					TSharedPtr<IDatasmithPropertyCaptureElement> LocationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					LocationProperty->SetCategory(EDatasmithPropertyCategory::RelativeLocation);
					LocationProperty->SetRecordedData((uint8*)&Location, sizeof(FVector));

					TSharedPtr<IDatasmithPropertyCaptureElement> RotationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					RotationProperty->SetCategory(EDatasmithPropertyCategory::RelativeRotation);
					RotationProperty->SetRecordedData((uint8*)&Rotation, sizeof(FRotator));

					TSharedPtr<IDatasmithPropertyCaptureElement> ScaleProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					ScaleProperty->SetCategory(EDatasmithPropertyCategory::RelativeScale3D);
					ScaleProperty->SetRecordedData((uint8*)&Scale, sizeof(FVector));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(CameraActor);
					Binding->AddPropertyCapture(LocationProperty.ToSharedRef());
					Binding->AddPropertyCapture(RotationProperty.ToSharedRef());
					Binding->AddPropertyCapture(ScaleProperty.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
			else
			{
				UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Did not spawn a camera actor, so camera variants were discarded!");
			}
			break;
		}
		case EDeltaGenVarDataVariantSwitchType::Geometry:
		{
			TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(*VarName);
			LVS->AddVariantSet(NewVarSet.ToSharedRef());
			CreatedVariantSetsToOriginalNames.Add(NewVarSet, VarName);

			TSet<TSharedPtr<IDatasmithActorElement>> NodeActors;
			for (const FName& NodeName : Var->Geometry.TargetNodes)
			{
				TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(NodeName);
				if (FoundActors)
				{
					NodeActors.Append(*FoundActors);
				}
				else
				{
					UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Did not find actor '%ls' used by variant '%ls'", *NodeName.ToString(), *VarName);
				}
			}
			TArray<TSharedPtr<IDatasmithActorElement>> NodeActorsArray = NodeActors.Array();

			for (const FDeltaGenVarDataGeometryVariant& Option : Var->Geometry.Variants)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*Option.Name);
				NewVarSet->AddVariant(NewVar.ToSharedRef());

				TSet<FName> VisibileNodes = TSet<FName>(Option.VisibleMeshes);

				for (const TSharedPtr<IDatasmithActorElement>& NodeActor : NodeActorsArray)
				{
					TSharedPtr<IDatasmithPropertyCaptureElement> PropertyCapture = FDatasmithSceneFactory::CreatePropertyCapture();
					PropertyCapture->SetCategory(EDatasmithPropertyCategory::Visibility);

					FName OriginalName = NodeActor->GetTagsCount() > 0? FName(NodeActor->GetTag(0)) : NAME_None;
					bool bVisible = VisibileNodes.Contains(OriginalName);
					PropertyCapture->SetRecordedData((uint8*)&bVisible, sizeof(bool));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(NodeActor);
					Binding->AddPropertyCapture(PropertyCapture.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
			break;
		}
		case EDeltaGenVarDataVariantSwitchType::ObjectSet:
		{
			TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(*VarName);
			LVS->AddVariantSet(NewVarSet.ToSharedRef());
			CreatedVariantSetsToOriginalNames.Add(NewVarSet, VarName);

			for (const FDeltaGenVarDataObjectSetVariant& Option : Var->ObjectSet.Variants)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*Option.Name);
				NewVarSet->AddVariant(NewVar.ToSharedRef());

				// Group up values by target actor name
				TMap<FName, TArray<int32>> ValueIndicesByTargetActor;
				for (int32 Index = 0; Index < Option.Values.Num(); ++Index)
				{
					TArray<int32>& Indices = ValueIndicesByTargetActor.FindOrAdd(Option.Values[Index].TargetNodeNameSanitized);
					Indices.Add(Index);
				}

				// Capture properties for each actor
				for (TPair<FName, TArray<int32>>& ValuePair : ValueIndicesByTargetActor)
				{
					FName& TargetActor = ValuePair.Key;
					TArray<int32>& Indices = ValuePair.Value;

					TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(TargetActor);
					if (!FoundActors || FoundActors->Num() == 0)
					{
						continue;
					}
					TSharedPtr<IDatasmithActorElement> TargetActorElement = (*FoundActors)[0];
					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(TargetActorElement);
					NewVar->AddActorBinding(Binding.ToSharedRef());

					// Capture each individual property
					for (int32 ValueIndex : Indices)
					{
						const FDeltaGenVarDataObjectSetVariantValue& Value = Option.Values[ValueIndex];

						TSharedPtr<IDatasmithPropertyCaptureElement> Prop = nullptr;

						// Create a property capture and record the data contained in Value
						switch (Value.DataType)
						{
						case EObjectSetDataType::Translation:
						{
							TSharedPtr<IDatasmithPropertyCaptureElement> LocationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
							LocationProperty->SetCategory(EDatasmithPropertyCategory::RelativeLocation);
							LocationProperty->SetRecordedData((uint8*)Value.Data.GetData(), sizeof(FVector));
							Prop = LocationProperty;
							break;
						}
						case EObjectSetDataType::Rotation:
						{
							TSharedPtr<IDatasmithPropertyCaptureElement> RotationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
							RotationProperty->SetCategory(EDatasmithPropertyCategory::RelativeRotation);
							RotationProperty->SetRecordedData((uint8*)Value.Data.GetData(), sizeof(FRotator));
							Prop = RotationProperty;
							break;
						}
						case EObjectSetDataType::Scaling:
						{
							TSharedPtr<IDatasmithPropertyCaptureElement> ScaleProperty = FDatasmithSceneFactory::CreatePropertyCapture();
							ScaleProperty->SetCategory(EDatasmithPropertyCategory::RelativeScale3D);
							ScaleProperty->SetRecordedData((uint8*)Value.Data.GetData(), sizeof(FVector));
							Prop = ScaleProperty;
							break;
						}
						case EObjectSetDataType::Visibility:
						{
							TSharedPtr<IDatasmithPropertyCaptureElement> VisibilityProperty = FDatasmithSceneFactory::CreatePropertyCapture();
							VisibilityProperty->SetCategory(EDatasmithPropertyCategory::Visibility);
							VisibilityProperty->SetRecordedData((uint8*)Value.Data.GetData(), sizeof(bool));
							Prop = VisibilityProperty;
							break;
						}
						case EObjectSetDataType::Center:
							break;
						default:
							UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Unsupported object set data type captured for actor '%ls' used by variant '%ls'", *TargetActor.ToString(), *VarName);
							continue;
						}

						if (Prop.IsValid())
						{
							Binding->AddPropertyCapture(Prop.ToSharedRef());
						}
					}
				}
			}
			break;
		}
		case EDeltaGenVarDataVariantSwitchType::SwitchObject:
		{
			FName SwitchObjectName = Var->SwitchObject.TargetSwitchObject;
			TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(SwitchObjectName);
			if (!FoundActors || FoundActors->Num() < 1)
			{
				UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Did not find actor '%ls' used by switch object variant '%ls'", *SwitchObjectName.ToString(), *VarName);
				break;
			}
			TSharedPtr<IDatasmithActorElement> SwitchObjectElement = (*FoundActors)[0];

			TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(*VarName);
			LVS->AddVariantSet(NewVarSet.ToSharedRef());
			CreatedVariantSetsToOriginalNames.Add(NewVarSet, VarName);

			TSharedPtr<IDatasmithActorElement> DisabledActor = nullptr;

			for (const FDeltaGenVarDataSwitchObjectVariant& Option : Var->SwitchObject.Variants)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*Option.Name);
				NewVarSet->AddVariant(NewVar.ToSharedRef());

				// DeltaGen uses the convention where if a switch is supposed to show nothing, it will just pick a selection index
				// that references nothing. Since our Switch Actors must always pick something, here we create an empty actor
				// to serve as the disabled alternative of the switch
				if (Option.Selection >= SwitchObjectElement->GetChildrenCount() && DisabledActor == nullptr)
				{
					if (TSharedPtr<IDatasmithActorElement>* FoundDisabledActor = CreatedSwitchObjectNullActors.Find(SwitchObjectElement))
					{
						DisabledActor = *FoundDisabledActor;
					}
					else
					{
						DisabledActor = FDatasmithSceneFactory::CreateActor(*UniqueNameProvider.GenerateUniqueName(*(VarName + TEXT("_Disabled"))));
						DisabledActor->AddTag(TEXT("Disabled"));
						CreatedSwitchObjectNullActors.Add(SwitchObjectElement, DisabledActor);
						SwitchObjectElement->AddChild(DisabledActor);
					}
				}

				for (int32 ChildIndex = 0; ChildIndex < SwitchObjectElement->GetChildrenCount(); ++ChildIndex)
				{
					TSharedPtr<IDatasmithActorElement> ChildActor = SwitchObjectElement->GetChild(ChildIndex);

					bool bIsVisible = false;
					if (ChildIndex == Option.Selection)
					{
						bIsVisible = true;
					}
					// In case the switch object has more than one selection index that reference nothing
					// Example: Originally 1 child, but has switch options "0", "1" and "2". We will only create one extra
					// "disabled" actor, so here we check if we're also in option "2"
					else if (Option.Selection >= SwitchObjectElement->GetChildrenCount() && ChildActor == DisabledActor)
					{
						bIsVisible = true;
					}

					TSharedPtr<IDatasmithPropertyCaptureElement> VisibilityProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					VisibilityProperty->SetCategory(EDatasmithPropertyCategory::Visibility);
					VisibilityProperty->SetRecordedData((uint8*)&bIsVisible, sizeof(bool));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(ChildActor);
					Binding->AddPropertyCapture(VisibilityProperty.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
			break;
		}
		default:
			UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Unsupported variant type for variant '%ls'", *VarName);
			continue;
		}
	}

	// Now that all non-package variants were parsed, can build the package variants to choose between them.
	// Package variants can point to other package variants, but DeltaGen doesn't allow circular references.
	// Also: We can only parse a package variant once we parsed its dependencies.
	// To parse them all, we'll scan from back to front, removing from PackageVariantsCopy every time we
	// successfully parse a package variant. When we reach index zero, we reset it to the max of the list and
	// check how many package variants we have still. If the number hasn't changed since the last reset, we
	// break and show an error message, as we're locked out
	TArray<FDeltaGenVarDataVariantSwitch*> PackageVariantsCopy = PackageVariants;
	int32 NumAtLastReset = -1;
	int32 PackageVarIndex = PackageVariantsCopy.Num() - 1;
	while (PackageVariantsCopy.Num() > 0)
	{
		FDeltaGenVarDataVariantSwitch* PackageVarSwitch = PackageVariantsCopy[PackageVarIndex];

		bool bCanBeParsedNow = true;

		// Parse the targets, as these are the same for every variant within this package
		TArray<TSharedPtr<IDatasmithVariantSetElement>> TargetVarSets;
		TArray<FDeltaGenVarDataVariantSwitch*> TargetDGVars;
		for (const FString& TargetVarSetName : PackageVarSwitch->Package.TargetVariantSets)
		{
			TSharedPtr<IDatasmithVariantSetElement> TargetVarSet = nullptr;
			for (const TPair<TSharedPtr<IDatasmithVariantSetElement>, FString>& VarSetPair : CreatedVariantSetsToOriginalNames)
			{
				if (TargetVarSetName == VarSetPair.Value)
				{
					TargetVarSet = VarSetPair.Key;
					break;
				}
			}

			// Check if our target is a package variant
			FDeltaGenVarDataVariantSwitch** TargetVarPtr = PackageVariants.FindByPredicate([&TargetVarSetName](const FDeltaGenVarDataVariantSwitch* Var)
			{
				return Var->Name == TargetVarSetName;
			});

			// If its a package variant and we haven't parsed it yet, step out
			if (TargetVarPtr && !TargetVarSet)
			{
				bCanBeParsedNow = false;
				break;
			}

			// If it's not a package variant, it's probably a regular variant
			if (!TargetVarPtr)
			{
				TargetVarPtr = NonPackageVariantsByName.Find(TargetVarSetName);
			}

			if (!TargetVarPtr)
			{
				UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Did not find the target DeltaGen variant '%ls' for parsed package variant '%ls'", *TargetVarSetName, *PackageVarSwitch->Name);
				// Can't step out/return as we need to keep the same number of targets as selected options
			}

			if (!TargetVarSet)
			{
				UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Did not find the target variant set '%ls' for parsed package variant '%ls'", *TargetVarSetName, *PackageVarSwitch->Name);
			}

			// Can't step out/return as we need to keep the same number of targets as selected options
			TargetDGVars.Add(TargetVarPtr? *TargetVarPtr : nullptr);
			TargetVarSets.Add(TargetVarSet);
		}

		if (bCanBeParsedNow)
		{
			TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(*PackageVarSwitch->Name);
			LVS->AddVariantSet(NewVarSet.ToSharedRef());
			CreatedVariantSetsToOriginalNames.Add(NewVarSet, PackageVarSwitch->Name);

			for (const FDeltaGenVarDataPackageVariant& Option : PackageVarSwitch->Package.Variants)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*Option.Name);
				NewVarSet->AddVariant(NewVar.ToSharedRef());

				int32 NumTargetVarSets = PackageVarSwitch->Package.TargetVariantSets.Num();
				for (int32 TargetIndex = 0; TargetIndex < NumTargetVarSets; TargetIndex++)
				{
					TSharedPtr<IDatasmithVariantSetElement> TargetVarSet = TargetVarSets[TargetIndex];
					FDeltaGenVarDataVariantSwitch* TargetDGVar = TargetDGVars[TargetIndex];
					if (!TargetVarSet.IsValid() || !TargetDGVar || Option.SelectedVariants.Num() != NumTargetVarSets)
					{
						// We gave a warning up there already
						continue;
					}

					// Find target UVariant
					int32 TargetVariantID = Option.SelectedVariants[TargetIndex];
					if (int32* TargetDGVarIndex = TargetDGVar->VariantIDToVariantIndex.Find(TargetVariantID))
					{
						TSharedPtr<IDatasmithVariantElement> VMVariant = TargetVarSet->GetVariant(*TargetDGVarIndex);
						if (!VMVariant.IsValid())
						{
							UE_LOGF(LogDatasmithDeltaGenImport, Warning, "When parsing package variant '%ls', did not find variant with index %d for variant set '%ls'", *PackageVarSwitch->Name, Option.SelectedVariants[TargetIndex], TargetVarSet->GetName());
							continue;
						}

						// Merge found UVariant into the variant we created, adding (but not overwriting) captures
						FDeltaGenVariantConverterImpl::MergeVariants(VMVariant, NewVar);
					}
				}
			}

			PackageVariantsCopy.RemoveAt(PackageVarIndex);
		}

		int32 RemainingPackageVariants = PackageVariantsCopy.Num();

		PackageVarIndex--;
		if (PackageVarIndex < 0 && RemainingPackageVariants > 0)
		{
			PackageVarIndex = RemainingPackageVariants - 1;

			if (RemainingPackageVariants == NumAtLastReset)
			{
				UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Failed to parse %d package variants due to too complex nested structure:", PackageVariantsCopy.Num());
				for (int32 Ind = 0; Ind < PackageVariantsCopy.Num(); Ind++)
				{
					UE_LOGF(LogDatasmithDeltaGenImport, Warning, "\t%ls", *PackageVariantsCopy[Ind]->Name);
				}
				break;
			}
			NumAtLastReset = RemainingPackageVariants;
		}
	}

	// Create a single variant set for all POS states, and each state becomes a variant
	if (PosStates.Num() > 0)
	{
		TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(TEXT("POS Variants"));
		LVS->AddVariantSet(NewVarSet.ToSharedRef());

		for (const FDeltaGenPosDataState& State : PosStates)
		{
			// Parse On/Off states
			for (const TPair<FString, bool>& StatePair : State.States)
			{
				FString ActorName = StatePair.Key;
				bool bOn = StatePair.Value;

				TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(*ActorName);
				if (!FoundActors)
				{
					UE_LOGF(LogDatasmithDeltaGenImport, Error, "Did not find actor '%ls' for POS state '%ls'", *ActorName, *State.Name);
					continue;
				}

				for (TSharedPtr<IDatasmithActorElement> FoundActor : *FoundActors)
				{
					TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*State.Name);
					NewVarSet->AddVariant(NewVar.ToSharedRef());

					TSharedPtr<IDatasmithPropertyCaptureElement> PropertyCapture = FDatasmithSceneFactory::CreatePropertyCapture();
					PropertyCapture->SetCategory(EDatasmithPropertyCategory::Visibility);
					PropertyCapture->SetRecordedData((uint8*)&bOn, sizeof(bool));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(FoundActor);
					Binding->AddPropertyCapture(PropertyCapture.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}

			// Parse switches
			for (const TPair<FName, int>& SwitchPair : State.Switches)
			{
				FString ActorName = SwitchPair.Key.ToString();
				int Selection = SwitchPair.Value;

				TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(FName(*ActorName));
				if (!FoundActors || FoundActors->Num() < 1)
				{
					UE_LOGF(LogDatasmithDeltaGenImport, Error, "Did not find actor '%ls' for POS state '%ls'", *ActorName, *State.Name);
					continue;
				}
				TSharedPtr<IDatasmithActorElement> ActorElement = (*FoundActors)[0];

				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*ActorName);
				NewVarSet->AddVariant(NewVar.ToSharedRef());

				for (int32 ChildIndex = 0; ChildIndex < ActorElement->GetChildrenCount(); ++ChildIndex)
				{
					bool bIsVisible = (ChildIndex == static_cast<int32>(Selection));

					TSharedPtr<IDatasmithPropertyCaptureElement> VisibilityProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					VisibilityProperty->SetCategory(EDatasmithPropertyCategory::Visibility);
					VisibilityProperty->SetRecordedData((uint8*)&bIsVisible, sizeof(bool));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(ActorElement->GetChild(ChildIndex));
					Binding->AddPropertyCapture(VisibilityProperty.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}

			// Parse materials
			for (const TPair<FString, FString>& MaterialPair : State.Materials)
			{
				FString ActorName = MaterialPair.Key;
				FString MaterialName = MaterialPair.Value;

				TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(FName(*ActorName));
				if (!FoundActors || FoundActors->Num() < 1)
				{
					UE_LOGF(LogDatasmithDeltaGenImport, Error, "Did not find actor '%ls' for POS state '%ls'", *ActorName, *State.Name);
					continue;
				}
				TSharedPtr<IDatasmithActorElement> ActorElement = (*FoundActors)[0];

				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*State.Name);
				NewVarSet->AddVariant(NewVar.ToSharedRef());

				TSharedPtr<IDatasmithBaseMaterialElement>* DatasmithMaterial = MaterialsByName.Find(FName(*MaterialName));
				if (!DatasmithMaterial || !DatasmithMaterial->IsValid())
				{
					UE_LOGF(LogDatasmithDeltaGenImport, Warning, "Failed to find material '%ls' for variant set '%ls'", *MaterialName, NewVarSet->GetName());
					continue;
				}

				TSharedPtr<IDatasmithObjectPropertyCaptureElement> PropertyCapture = FDatasmithSceneFactory::CreateObjectPropertyCapture();
				PropertyCapture->SetCategory(EDatasmithPropertyCategory::Material);
				PropertyCapture->SetRecordedObject(*DatasmithMaterial);

				TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
				Binding->SetActor(ActorElement);
				Binding->AddPropertyCapture(PropertyCapture.ToSharedRef());

				NewVar->AddActorBinding(Binding.ToSharedRef());
			}
		}
	}

	return LVS;
}
