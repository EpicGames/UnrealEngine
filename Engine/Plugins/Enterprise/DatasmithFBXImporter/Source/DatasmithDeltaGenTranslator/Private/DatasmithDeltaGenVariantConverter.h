// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#define SCENECAMERA_NAME TEXT("Camera")

class AActor;
class IDatasmithActorElement;
class IDatasmithBaseMaterialElement;
class IDatasmithLevelVariantSetsElement;
struct FAssetData;
struct FDeltaGenPosDataState;
struct FDeltaGenVarDataVariantSwitch;

using FActorMap = TMap<FName, TArray<TSharedPtr<IDatasmithActorElement>>>;
using FMaterialMap = TMap<FName, TSharedPtr<IDatasmithBaseMaterialElement>>;

class FDeltaGenVariantConverter
{
public:
	static TSharedPtr<IDatasmithLevelVariantSetsElement> ConvertVariants(TArray<FDeltaGenVarDataVariantSwitch>& Vars, TArray<FDeltaGenPosDataState>& PosStates, FActorMap& ActorsByOriginalName, FMaterialMap& MaterialsByName);
};

