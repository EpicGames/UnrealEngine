// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#define DELTAGENTRANSLATOR_MODULE_NAME TEXT("DatasmithDeltaGenTranslator")

class IDatasmithDeltaGenTranslatorModule : public IModuleInterface
{
public:
	static inline IDatasmithDeltaGenTranslatorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IDatasmithDeltaGenTranslatorModule >(DELTAGENTRANSLATOR_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(DELTAGENTRANSLATOR_MODULE_NAME);
	}
};
