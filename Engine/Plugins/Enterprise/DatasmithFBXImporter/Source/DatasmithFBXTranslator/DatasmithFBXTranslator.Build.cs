// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithFBXTranslator : ModuleRules
	{
		public DatasmithFBXTranslator(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DatasmithCore",
					"EditorFramework",
					"Engine",
					"LevelSequence",
					"MeshDescription",
					"StaticMeshDescription",
					"UnrealEd", // For UnFbx
				}
			);
			AddEngineThirdPartyPrivateStaticDependencies(Target,
				"FBX"
			);
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithContent",
					"DatasmithTranslator"
				}
			);
		}
	}
}
