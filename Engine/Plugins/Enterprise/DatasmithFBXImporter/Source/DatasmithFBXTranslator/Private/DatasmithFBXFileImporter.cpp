// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFBXFileImporter.h"
#include "DatasmithDefinitions.h"
#include "DatasmithFBXImporterLog.h"
#include "DatasmithFBXImportOptions.h"
#include "DatasmithFBXScene.h"
#include "DatasmithUtils.h"
#include "Utility/DatasmithMeshHelper.h"

#include "Async/Async.h"
#include "FbxImporter.h"
#include "StaticMeshAttributes.h"
#include "Misc/Paths.h"
#include "RenderMath.h"
#include <core/base/fbxtime.h>
#include <core/fbxclassid.h>
#include <core/fbxdatatypes.h>
#include <core/fbxobject.h>
#include <core/fbxproperty.h>
#include <core/fbxpropertydef.h>
#include <core/fbxquery.h>
#include <core/math/fbxquaternion.h>
#include <core/math/fbxvector2.h>
#include <exception>
#include <fileio/fbxglobalsettings.h>
#include <scene/animation/fbxanimcurve.h>
#include <scene/animation/fbxanimcurvenode.h>
#include <scene/animation/fbxanimevaluator.h>
#include <scene/animation/fbxanimlayer.h>
#include <scene/animation/fbxanimstack.h>
#include <scene/fbxscene.h>
#include <scene/geometry/fbxcamera.h>
#include <scene/geometry/fbxlayer.h>
#include <scene/geometry/fbxlight.h>
#include <scene/geometry/fbxmesh.h>
#include <scene/shading/fbxtexture.h>
#include <stdexcept>
#include <utils/fbxgeometryconverter.h>
#include <utils/fbxrenamingstrategyutilities.h>
#include <utils/fbxrootnodeutility.h>

#define ROOT_NODE_NAME TEXT("RootNode")
#define INCH_TO_MM(value) (value * 25.4)
#define SAFE_FLT(x) FMath::Clamp(x, -MAX_FLT, MAX_FLT); // Turns nans and -inf into -FLT_MAX and +inf into FLT_MAX
#define SAFE_TANGENT_THRESHOLD 300.0f
#define SAFE_TANGENT(x) FMath::Clamp(x, -SAFE_TANGENT_THRESHOLD, SAFE_TANGENT_THRESHOLD);
#define TIME_OF_TAG_KEY 100  // Where in an animation curve we insert a tag

namespace FBXFileImporterImpl
{
	FVector ConvertPos(FbxVector4 Vector)
	{
		FVector Out;
		Out[0] = Vector[0];
		// flip Y, then the right-handed axis system is converted to LHS
		Out[1] = -Vector[1];
		Out[2] = Vector[2];
		return Out;
	}

	FVector ConvertDir(FbxVector4 Vector)
	{
		FVector Out;
		Out[0] = Vector[0];
		Out[1] = -Vector[1];
		Out[2] = Vector[2];
		return Out;
	}

	FVector ConvertScale(FbxDouble3 Vector)
	{
		FVector Out;
		Out[0] = Vector[0];
		Out[1] = Vector[1];
		Out[2] = Vector[2];
		return Out;
	}


	FVector ConvertScale(FbxVector4 Vector)
	{
		FVector Out;
		Out[0] = Vector[0];
		Out[1] = Vector[1];
		Out[2] = Vector[2];
		return Out;
	}

	FQuat ConvertRotToQuat(FbxQuaternion Quaternion)
	{
		FQuat UnrealQuat;
		UnrealQuat.X = Quaternion[0];
		UnrealQuat.Y = -Quaternion[1];
		UnrealQuat.Z = Quaternion[2];
		UnrealQuat.W = -Quaternion[3];
		return UnrealQuat;
	}

	// Copied from FbxMainImport.cpp
	FString GetFbxPropertyStringValue( const FbxProperty& Property )
	{
		FString ValueStr( TEXT( "Unsupported type" ) );

		FbxDataType DataType = Property.GetPropertyDataType();
		switch ( DataType.GetType() )
		{
		case eFbxBool:
		{
			FbxBool BoolValue = Property.Get<FbxBool>();
			ValueStr = LexToString( BoolValue );
		}
		break;
		case eFbxChar:
		{
			FbxChar CharValue = Property.Get<FbxChar>();
			ValueStr = LexToString( CharValue );
		}
		break;
		case eFbxUChar:
		{
			FbxUChar UCharValue = Property.Get<FbxUChar>();
			ValueStr = LexToString( UCharValue );
		}
		break;
		case eFbxShort:
		{
			FbxShort ShortValue = Property.Get<FbxShort>();
			ValueStr = LexToString( ShortValue );
		}
		break;
		case eFbxUShort:
		{
			FbxUShort UShortValue = Property.Get<FbxUShort>();
			ValueStr = LexToString( UShortValue );
		}
		break;
		case eFbxInt:
		{
			FbxInt IntValue = Property.Get<FbxInt>();
			ValueStr = LexToString( IntValue );
		}
		break;
		case eFbxUInt:
		{
			FbxUInt UIntValue = Property.Get<FbxUInt>();
			ValueStr = LexToString( UIntValue );
		}
		break;
		case eFbxEnum:
		{
			FbxEnum EnumValue = Property.Get<FbxEnum>();
			ValueStr = LexToString( EnumValue );
		}
		break;
		case eFbxFloat:
		{
			FbxFloat FloatValue = Property.Get<FbxFloat>();
			ValueStr = LexToString( FloatValue );
		}
		break;
		case eFbxDouble:
		{
			FbxDouble DoubleValue = Property.Get<FbxDouble>();
			ValueStr = LexToString( DoubleValue );
		}
		break;
		case eFbxDouble2:
		{
			FbxDouble2 Vec = Property.Get<FbxDouble2>();
			ValueStr = FString::Printf( TEXT( "(%f, %f)" ), Vec[ 0 ], Vec[ 1 ] );
		}
		break;
		case eFbxDouble3:
		{
			FbxDouble3 Vec = Property.Get<FbxDouble3>();
			ValueStr = FString::Printf( TEXT( "(%f, %f, %f)" ), Vec[ 0 ], Vec[ 1 ], Vec[ 2 ] );
		}
		break;
		case eFbxDouble4:
		{
			FbxDouble4 Vec = Property.Get<FbxDouble4>();
			ValueStr = FString::Printf( TEXT( "(%f, %f, %f, %f)" ), Vec[ 0 ], Vec[ 1 ], Vec[ 2 ], Vec[ 3 ] );
		}
		break;
		case eFbxString:
		{
			FbxString StringValue = Property.Get<FbxString>();
			ValueStr = UTF8_TO_TCHAR( StringValue.Buffer() );
		}
		break;
		default:
			break;
		}
		return ValueStr;
	}
}

FDatasmithFBXFileImporter::FDatasmithFBXFileImporter(FbxScene* InFbxScene, FDatasmithFBXScene* InScene, const UDatasmithFBXImportOptions* InOptions, const FDatasmithImportBaseOptions* InBaseOptions)
	: InScene(InFbxScene)
	, OutScene(InScene)
	, Options(InOptions)
	, BaseOptions(InBaseOptions)
{
}

void FDatasmithFBXFileImporter::ImportScene()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithFBXFileImporter::ImportScene)

	ConvertCoordinateSystem();

	// ImportMaterials
	// import all materials contained in the scene - any might be referenced in var files
	if(BaseOptions->bIncludeMaterial)
	{
		FbxArray<FbxSurfaceMaterial*> MaterialArray;
		InScene->FillMaterialArray(MaterialArray);

		TArray<FString> MaterialNames;
		for(int MaterialIndex = 0; MaterialIndex < MaterialArray.Size(); ++MaterialIndex)
		{
			FbxSurfaceMaterial* Material = MaterialArray[MaterialIndex];
			ImportMaterial(Material);
		}

		TSet<TSharedPtr<FDatasmithFBXSceneMaterial>> UniqueMaterials;
		for(auto MaterialFbx: ImportedMaterials)
		{
			UniqueMaterials.Add(MaterialFbx.Value);
		}

		OutScene->Materials = UniqueMaterials.Array();
	}

	// We use the RootNode to apply our scale conversion now. If we ask the FBX SDK to manually convert
	// the scene, all it will do is place scaling factors on all top level nodes. If those are animated
	// it becomes really messy to apply the correct scaling factors given that the animations may come
	// from .tml files (DeltaGen) or the FBX scene itself (VRED)
	OutScene->RootNode = TSharedPtr<FDatasmithFBXSceneNode>(new FDatasmithFBXSceneNode());
	OutScene->RootNode->Name = ROOT_NODE_NAME;
	OutScene->RootNode->OriginalName = ROOT_NODE_NAME;
	OutScene->RootNode->SplitNodeID = 0;
	OutScene->RootNode->LocalTransform.SetScale3D(FVector(OutScene->ScaleFactor));
	OutScene->RootNode->bShouldKeepThisNode = true;

	ImportedAnimationCurves.Reset();
	for (FDatasmithFBXSceneAnimNode& AnimNode : OutScene->AnimNodes)
	{
		for (FDatasmithFBXSceneAnimBlock& Block : AnimNode.Blocks)
		{
			for (FDatasmithFBXSceneAnimCurve& Curve : Block.Curves)
			{
				int32 DSID = Curve.DSID;
				ImportedAnimationCurves.FindOrAdd(DSID).Add(&Curve);
			}
		}
	}

	// Import scene hierarchy, extracting animations and creating placeholders for all meshes
	TraverseHierarchyNodeRecursively(InScene->GetRootNode(), OutScene->RootNode);

	TArray<TFuture<FbxMesh*>> Tasks;
	Tasks.Reserve(ImportedMeshes.Num());

	// Import meshes
	for (auto& It : ImportedMeshes)
	{
		Tasks.Emplace(
			Async(
				EAsyncExecution::LargeThreadPool,
				[FbxMeshPtr = It.Key, DatasmithFBXSceneMeshPtr = It.Value.Get()]()
				{
					DoImportMesh(FbxMeshPtr, DatasmithFBXSceneMeshPtr);
					return FbxMeshPtr;
				}
			)
		);
	}

	for (TFuture<FbxMesh*>& Future : Tasks)
	{
		// Destroy Fbx mesh to save memory - it won't be used anymore. Destroying an FbxMesh will unlink it from all places
		// in Fbx scene, i.e. all instances will be lost. However we're calling DoImportMesh() when we already have full
		// Fbx hierarchy imported into scene structures, so it won't damage anything.
		Future.Get()->Destroy();
	}
}

void FDatasmithFBXFileImporter::ConvertCoordinateSystem()
{
	// We will place this ScaleFactor on the root node now
	OutScene->ScaleFactor = InScene->GetGlobalSettings().GetSystemUnit().GetScaleFactor();

	// We use -Y as forward axis here when we import. This is odd considering our forward axis is technically +X
	// but this is to mimic Maya/Max behavior where if you make a model facing +X facing, when you import that mesh,
	// you want +X facing in engine. Only thing that doesn't work is hand flipping because Max/Maya is RHS but UE is LHS
	// On the positive note, we now have import transform set up you can do to rotate mesh if you don't like default setting.
	FbxAxisSystem::ECoordSystem CoordSystem = FbxAxisSystem::eRightHanded;
	FbxAxisSystem::EUpVector UpVector = FbxAxisSystem::eZAxis;
	FbxAxisSystem::EFrontVector FrontVector = (FbxAxisSystem::EFrontVector) - FbxAxisSystem::eParityOdd;

	FbxAxisSystem UnrealImportAxis(UpVector, FrontVector, CoordSystem);
	FbxAxisSystem SourceSetup = InScene->GetGlobalSettings().GetAxisSystem();

	if (SourceSetup != UnrealImportAxis)
	{
		FbxRootNodeUtility::RemoveAllFbxRoots(InScene);
		UnrealImportAxis.ConvertScene(InScene);
	}
}

void FDatasmithFBXFileImporter::TraverseHierarchyNodeRecursively(FbxNode* ParentNode, TSharedPtr<FDatasmithFBXSceneNode>& ParentInfo)
{
	int32 NodeCount = ParentNode->GetChildCount();
	for (int32 NodeIndex = 0; NodeIndex < NodeCount; NodeIndex++)
	{
		//? Process visibility attributes here. Note: when material libraries are stored in
		//? invisible nodes, we still should import them, because these materials may be used
		//? in variants (referenced there by name). Probably store all materials in Scene object.

		FbxNode* ChildNode = ParentNode->GetChild(NodeIndex);
		TSharedPtr<FDatasmithFBXSceneNode> ChildInfo(new FDatasmithFBXSceneNode());

		// Process node name
		FString NodeName = UTF8_TO_TCHAR(ChildNode->GetName());
		// Truncate any Fbx name clash suffixes
		int32 Pos = NodeName.Find(TEXT(NAMECLASH1_KEY));
		if (Pos != INDEX_NONE)
		{
			NodeName = NodeName.Left(Pos);
		}
		ChildInfo->Name = FDatasmithUtils::SanitizeObjectName(NodeName);
		ChildInfo->OriginalName = NodeName;

		if (ChildInfo->Name == TEXT("SceneRoot"))
		{
			// This is a reserved name, don't use it, otherwise plugin will crash when creating components.
			ChildInfo->Name.Append(TEXT("_1"));
		}

		if (BaseOptions->bIncludeAnimation)
		{
			// If the node is animated, this will extract its animations to our ImportedAnimationData member
			ExtractAnimations(ChildNode);
		}
		ChildInfo->Visibility = ChildNode->Visibility.IsValid() ? ChildNode->Visibility.Get() : 1.0f;
		ChildInfo->bVisibilityInheritance = !ChildNode->VisibilityInheritance.IsValid() || ChildNode->VisibilityInheritance.Get();

		// Convert node attributes

		// Transform
		FVector RotOffset = FBXFileImporterImpl::ConvertPos(ChildNode->GetRotationOffset(FbxNode::EPivotSet::eSourcePivot));
		FVector ScaleOffset = FBXFileImporterImpl::ConvertPos(ChildNode->GetScalingOffset(FbxNode::EPivotSet::eSourcePivot));
		FVector RotPivot = FBXFileImporterImpl::ConvertPos(ChildNode->GetRotationPivot(FbxNode::EPivotSet::eSourcePivot));
		FVector ScalePivot = FBXFileImporterImpl::ConvertPos(ChildNode->GetScalingPivot(FbxNode::EPivotSet::eSourcePivot));

		FbxVector4& LocalTrans = InScene->GetAnimationEvaluator()->GetNodeLocalTranslation(ChildNode);
		FbxVector4& LocalScale = InScene->GetAnimationEvaluator()->GetNodeLocalScaling(ChildNode);
		FbxVector4& LocalRot = InScene->GetAnimationEvaluator()->GetNodeLocalRotation(ChildNode);

		FbxAMatrix Temp;
		Temp.SetIdentity();
		Temp.SetR(LocalRot);
		FbxQuaternion LocalRotQuat = Temp.GetQ();

		FVector4 Trans = FBXFileImporterImpl::ConvertPos(LocalTrans);
		FVector4 Scale = FBXFileImporterImpl::ConvertScale(LocalScale);
		FQuat Rotation = FBXFileImporterImpl::ConvertRotToQuat(LocalRotQuat);

		FVector RotEuler = Rotation.Euler();

		// Avoid singularity around 90 degree pitch, as UnrealEditor doesn't seem to support it very well
		// See UE-75467 and UE-83049
		if (FMath::IsNearlyEqual(abs(RotEuler.Y), (FVector::FReal)90.0f))
		{
			Rotation.W += 1e-3;
			Rotation.Normalize();
		}

		// Converting exactly 180.0 degree quaternions into Euler is unreliable, so add some
		// small noise so that it produces the correct actor transform
		if (abs(RotEuler.X) == 180.0f ||
			abs(RotEuler.Y) == 180.0f ||
			abs(RotEuler.Z) == 180.0f)
		{
			Rotation.W += 1.e-7;
			Rotation.Normalize();
		}

		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "Node: %ls", *ChildInfo->Name);
		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "\tTranslation: [%f, %f, %f]", Trans.X, Trans.Y, Trans.Z);
		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "\tScale: [%f, %f, %f]", Scale.X, Scale.Y, Scale.Z);
		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "\tRotation: [%f, %f, %f, %f]", Rotation.X, Rotation.Y, Rotation.Z, Rotation.W);
		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "\tRotationPivot: [%f, %f, %f]", RotPivot.X, RotPivot.Y, RotPivot.Z);
		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "\tScalingPivot: [%f, %f, %f]", ScalePivot.X, ScalePivot.Y, ScalePivot.Z);
		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "\tRotationOffset: [%f, %f, %f]", RotOffset.X, RotOffset.Y, RotOffset.Z);
		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "\tScalingOffset: [%f, %f, %f]", ScaleOffset.X, ScaleOffset.Y, ScaleOffset.Z);

		ChildInfo->RotationOffset = RotOffset;
		ChildInfo->ScalingOffset = ScaleOffset;
		ChildInfo->RotationPivot = RotPivot - ScaleOffset;
		ChildInfo->ScalingPivot = ScalePivot;// - RotOffset; I do not know why this is not necessary. Maybe due to how we split rotation then scaling?

		FTransform NewTransform;
		NewTransform.SetTranslation(Trans + RotOffset + ScaleOffset);
		NewTransform.SetScale3D(Scale);
		NewTransform.SetRotation(Rotation);
		ChildInfo->LocalTransform = NewTransform;
		if( !ChildInfo->LocalTransform.IsValid() )
		{
			ChildInfo->LocalTransform.SetIdentity();
		}

		FbxNodeAttribute* FbxAttr = ChildNode->GetNodeAttribute();
		if (FbxAttr != nullptr)
		{
			FbxNodeAttribute::EType FbxAttrType = FbxAttr->GetAttributeType();
			if (FbxAttrType == FbxNodeAttribute::eCamera && BaseOptions->bIncludeCamera)
			{
				FbxCamera* Camera = static_cast<FbxCamera*>(FbxAttr);
				TSharedPtr<FDatasmithFBXSceneCamera> SceneCamera = MakeShared<FDatasmithFBXSceneCamera>();

				double ConversionFactor = InScene->GetGlobalSettings().GetSystemUnit().GetScaleFactor();

				// These always come in inches according to the FBX SDK
				SceneCamera->SensorWidth = INCH_TO_MM(Camera->FilmWidth.Get());
				SceneCamera->SensorAspectRatio = Camera->FilmWidth.Get() / Camera->FilmHeight.Get();

				// These are always in mm according to the FBX SDK
				SceneCamera->FocalLength = Camera->FocalLength.Get();
				SceneCamera->FocusDistance = Camera->FocusDistance.Get();

				// These come in VRED's scale units, which need to be converted to cm
				// They don't seem to actually be used for anything though
				SceneCamera->NearPlane = Camera->NearPlane.Get() * ConversionFactor;
				SceneCamera->FarPlane = Camera->FarPlane.Get() * ConversionFactor;
				SceneCamera->OrthoZoom = Camera->OrthoZoom.Get() * ConversionFactor;

				SceneCamera->Roll = Camera->FilmRollValue.Get();

				if (Camera->ProjectionType.Get() == fbxsdk::FbxCamera::EProjectionType::eOrthogonal)
				{
					SceneCamera->ProjectionType = EProjectionType::Orghographic;
				}
				else
				{
					SceneCamera->ProjectionType = EProjectionType::Perspective;
				}

				ChildInfo->Camera = SceneCamera;
			}
			else if (FbxAttrType == FbxNodeAttribute::eLight && BaseOptions->bIncludeLight)
			{
				FbxLight* Light = static_cast<FbxLight*>(FbxAttr);
				TSharedPtr<FDatasmithFBXSceneLight> SceneLight = MakeShared<FDatasmithFBXSceneLight>();

				FbxDouble3 lightColor = Light->Color.Get();

				SceneLight->Name = NodeName;
				SceneLight->DiffuseColor = FLinearColor(lightColor[0], lightColor[1], lightColor[2]);
				SceneLight->UseIESProfile = false;

				// By default FBX doesn't allow temperature. It will still work though, since VRED is nice enough
				// to convert the temperature into light color, and the intensity works normally
				SceneLight->UseTemperature = false;
				SceneLight->Temperature = 6500.0f;

				SceneLight->Intensity = Light->Intensity.Get();
				SceneLight->Enabled = Light->CastLight.Get();
				SceneLight->ConeInnerAngle = Light->InnerAngle.Get();
				SceneLight->ConeOuterAngle = Light->OuterAngle.Get();

				// Convert attenuation and intensities
				switch (Light->DecayType.Get())
				{
					case FbxLight::EDecayType::eNone:
						SceneLight->AttenuationType = EAttenuationType::None;
						// VRED uses "light intensity = 1.0" as corresponding to 400 lumens. It then writes candelas to
						// the FBX. This means a VRED light intensity 1.0 will show up as 31.830988 in the FBX,
						// which is why we compensate here. For the other attenuation types VRED also applies a 100x and 10000x
						// multiplier
						SceneLight->Intensity /= 31.830988f;
						break;
					case FbxLight::EDecayType::eLinear:
						SceneLight->AttenuationType = EAttenuationType::Linear;
						SceneLight->Intensity /= 3183.098877f;
						break;
					case FbxLight::EDecayType::eQuadratic:
					case FbxLight::EDecayType::eCubic: //Fall through since there is no cubic decay in VRED
						SceneLight->AttenuationType = EAttenuationType::Realistic;
						SceneLight->Intensity /= 318309.875f;
						break;
				}

				// VRED never packs area light intensities in the FBX, sadly. Let's reset what we did up there
				// since regardless of attenuation, for area lights the intensity values we were reading were
				// just defaults anyway
				if (Light->LightType.Get() == FbxLight::EType::eArea)
				{
					SceneLight->Intensity = 1.0f;
				}

				// Set light type and shape
				switch (Light->LightType.Get())
				{
					case FbxLight::EType::ePoint:
						SceneLight->LightType = ELightType::Point;
						break;
					case FbxLight::EType::eDirectional:
						SceneLight->LightType = ELightType::Directional;
						break;
					case FbxLight::EType::eArea:
					{
						SceneLight->LightType = ELightType::Area;

						// Get light shape. The FBX format really restricts those, we'll need to import
						// extra info if we want other shapes
						switch (Light->AreaLightShape.Get())
						{
							case fbxsdk::FbxLight::EAreaLightShape::eRectangle:
								SceneLight->AreaLightShape = EDatasmithLightShape::Rectangle;
								break;
							default:
								SceneLight->AreaLightShape = EDatasmithLightShape::Sphere;
								break;
						}
						break;
					}
					case FbxLight::EType::eSpot:
						SceneLight->LightType = ELightType::Spot;
						break;
				}

				ChildInfo->Light = SceneLight;
			}
			else if (FbxAttrType == FbxNodeAttribute::eMesh && BaseOptions->bIncludeGeometry)
			{
				// This is a mesh
				FbxMesh* Mesh = static_cast<FbxMesh*>(FbxAttr);
				if (!Mesh->IsTriangleMesh())
				{
					// Triangulation stuff
					UE_LOGF(LogDatasmithFBXImport, Warning, "Triangulating static mesh %ls", *ChildInfo->Name);
					const bool bReplace = true;
					FbxNodeAttribute* ConvertedNode = UnFbx::FFbxImporter::GetInstance()->GetGeometryConverter()->Triangulate(Mesh, bReplace);
					if (ConvertedNode != nullptr && ConvertedNode->GetAttributeType() == FbxNodeAttribute::eMesh)
					{
						Mesh = static_cast<FbxMesh*>(ConvertedNode);
					}
					else
					{
						UE_LOGF(LogDatasmithFBXImport, Error, "Unable to triangulate mesh '%ls'", *ChildInfo->Name);
						Mesh = nullptr;
					}
				}

				if (Mesh != nullptr && Mesh->GetPolygonCount() != 0)
				{
					// Perform mesh import. Do not import empty meshes.
					TSharedPtr<FDatasmithFBXSceneMesh> SceneMesh = ImportMesh(Mesh, ChildNode);
					ChildInfo->Mesh = SceneMesh;

					// When we have a mesh, we should import its materials too. Note: we're storing materials
					// in node too, to allow multiple instances of the same mesh with different materials.

					// Note: FbxSceneImporter plugin doesn't care about different materials assigned to the same mesh, so second
					// material assignment will be lost. Our plugin does all the job correctly, even if this is not needed.

					if (BaseOptions->bIncludeMaterial)
					{
						int32 NumMaterials = ChildNode->GetMaterialCount();
						if (!SceneMesh->ImportMaterialsNode.IsValid())
						{
							// In DeltaGen/VRED Fbx files we often have a mesh attached to some node which has no materials.
							// In this case we have the same mesh used somewhere else with assigned materials. The code
							// below allows to use these materials. Note that first instance in Fbx will always have
							// materials, so we don't need to care that node with materials will appear after a node
							// without materials.
							SceneMesh->ImportMaterialsNode = ChildInfo;
						}

						if (NumMaterials != 0)
						{
							// Node has material information
							for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; MaterialIndex++)
							{
								ChildInfo->Materials.Add(ImportMaterial(ChildNode->GetMaterial(MaterialIndex)));
							}
						}
						else if (SceneMesh->ImportMaterialsNode.IsValid())
						{
							// The node has no materials, but we have another instance which has ones
							ChildInfo->Materials = SceneMesh->ImportMaterialsNode.Pin()->Materials;
						}
					}
				}
			}
		}

		// Import all custom user-defined FBX properties from the FBX node to the object metadata
		FbxProperty CurrentProperty = ChildNode->GetFirstProperty();
		while ( CurrentProperty.IsValid() )
		{
			if ( CurrentProperty.GetFlag( FbxPropertyFlags::eUserDefined ) )
			{
				FString MetadataTag = UTF8_TO_TCHAR( CurrentProperty.GetName() );
				FString MetadataValue = FBXFileImporterImpl::GetFbxPropertyStringValue( CurrentProperty );

				ChildInfo->Metadata.Add( MetadataTag, MetadataValue );
			}
			CurrentProperty = ChildNode->GetNextProperty( CurrentProperty );
		}

		// Process hierarchy
		ParentInfo->Children.Add(ChildInfo);
		ChildInfo->Parent = ParentInfo;
		TraverseHierarchyNodeRecursively(ChildNode, ChildInfo);
	}
}

void ProcessCurve(FbxAnimCurve* InCurve, FDatasmithFBXSceneAnimCurve* OutCurve, EDatasmithFBXSceneAnimationCurveType InType, EDatasmithFBXSceneAnimationCurveComponent InComponent)
{
	if (InCurve == nullptr)
	{
		UE_LOGF(LogDatasmithFBXImport, Warning, "InCurve was nullptr!");
		return;
	}
	if (OutCurve == nullptr)
	{
		UE_LOGF(LogDatasmithFBXImport, Warning, "OutCurve was nullptr!");
		return;
	}

	// Correction due to different handedness between coordinate systems
	float Correction = 1.0f;
	if ((InType == EDatasmithFBXSceneAnimationCurveType::Translation && InComponent == EDatasmithFBXSceneAnimationCurveComponent::Y) ||
		(InType == EDatasmithFBXSceneAnimationCurveType::Rotation && (InComponent == EDatasmithFBXSceneAnimationCurveComponent::Y || InComponent == EDatasmithFBXSceneAnimationCurveComponent::Z)))
	{
		Correction = -1.0f;
	}

	int KeyCount = InCurve->KeyGetCount();
	OutCurve->Points.Reserve(KeyCount);

	// Pack keys and values into animation track
	for (int32 KeyIndex = 0; KeyIndex < KeyCount; KeyIndex++)
	{
		FbxAnimCurveKey Key = InCurve->KeyGet(KeyIndex);
		float KeyTime = SAFE_FLT((float)Key.GetTime().GetSecondDouble());

		UE_LOGF(LogDatasmithFBXImport, VeryVerbose, "Time: %f, Value: %f", KeyTime, Key.GetValue());

		float ArriveTangent = 0.0f;
		float LeaveTangent = 0.0f;

		// Calculate arrive/leave tangents for all keys
		{
			float LeftTangent = InCurve->KeyGetLeftDerivative(KeyIndex);
			float RightTangent = InCurve->KeyGetRightDerivative(KeyIndex);

			if (KeyIndex > 0)
			{
				ArriveTangent = LeftTangent *
					(Key.GetTime().GetSecondDouble() - InCurve->KeyGetTime(KeyIndex - 1).GetSecondDouble());
			}

			if (KeyIndex < KeyCount - 1)
			{
				LeaveTangent = RightTangent *
					(InCurve->KeyGetTime(KeyIndex + 1).GetSecondDouble() - Key.GetTime().GetSecondDouble());
			}
		}

		// VRED/FBX flips the sign of the LeftDerivative if it's infinite
		if (ArriveTangent > MAX_FLT || ArriveTangent < -MAX_FLT)
		{
			ArriveTangent *= -1;
		}

		if (FMath::Abs(ArriveTangent) > SAFE_TANGENT_THRESHOLD)
		{
			UE_LOGF(LogDatasmithFBXImport, Warning, "Key %d of animation track type %d, component %d will have its ArriveTangent clamped to the value %f", KeyIndex, (int32)InType, (int32)InComponent, ArriveTangent > 0 ? SAFE_TANGENT_THRESHOLD : -SAFE_TANGENT_THRESHOLD);
		}

		if (FMath::Abs(LeaveTangent) > SAFE_TANGENT_THRESHOLD)
		{
			UE_LOGF(LogDatasmithFBXImport, Warning, "Key %d of animation track type %d, component %d will have its LeaveTangent clamped to the value %f", KeyIndex, (int32)InType, (int32)InComponent, LeaveTangent > 0 ? SAFE_TANGENT_THRESHOLD : -SAFE_TANGENT_THRESHOLD);
		}

		ERichCurveInterpMode InterpMode;
		switch (Key.GetInterpolation())
		{
		case FbxAnimCurveDef::eInterpolationConstant:
			InterpMode = ERichCurveInterpMode::RCIM_Constant;
			break;
		case FbxAnimCurveDef::eInterpolationLinear:
			InterpMode = ERichCurveInterpMode::RCIM_Linear;
			break;
		case FbxAnimCurveDef::eInterpolationCubic:
			InterpMode = ERichCurveInterpMode::RCIM_Cubic;
			break;
		default:
			InterpMode = ERichCurveInterpMode::RCIM_None;
			break;
		}

		ERichCurveTangentMode TangentMode;
		switch (Key.GetTangentMode(true))
		{
		case FbxAnimCurveDef::eTangentUser: // Intended fallthrough
		case FbxAnimCurveDef::eTangentTCB: // Intended fallthrough
		case FbxAnimCurveDef::eTangentGenericClamp: // Intended fallthrough
		case FbxAnimCurveDef::eTangentGenericClampProgressive:
			TangentMode = ERichCurveTangentMode::RCTM_User;
			break;
		case FbxAnimCurveDef::eTangentAuto:
			TangentMode = ERichCurveTangentMode::RCTM_Auto;
			break;
		case FbxAnimCurveDef::eTangentGenericBreak:
		default:
			TangentMode = ERichCurveTangentMode::RCTM_Break;
			break;
		}

		// Preserve sharp corners when the curve arrives sharply with cubic interpolation
		// at a node that has linear interpolation. Not doing this and letting the tangent mode go
		// to Auto would force equal tangents on both sides
		if (!FMath::IsNearlyEqual(ArriveTangent, LeaveTangent))
		{
			TangentMode = ERichCurveTangentMode::RCTM_Break;
		}

		FDatasmithFBXSceneAnimPoint* NewPoint = new(OutCurve->Points) FDatasmithFBXSceneAnimPoint;
		NewPoint->Time = KeyTime;
		NewPoint->Value = SAFE_FLT(Key.GetValue() * Correction);
		NewPoint->ArriveTangent = SAFE_TANGENT(ArriveTangent);
		NewPoint->LeaveTangent = SAFE_TANGENT(LeaveTangent);
		NewPoint->InterpolationMode = InterpMode;
		NewPoint->TangentMode = TangentMode;
	}
}

/**
 * Tries to extract key frames from InCurve, building a new FDatasmithFBXSceneAnimCurve of type InType and component InComponent if it can.
 * It does a lot of math to convert coordinate systems and correct the interpolation between keys
 */
void FDatasmithFBXFileImporter::FillCurveFromClipsFile(FbxAnimCurve* InCurve, EDatasmithFBXSceneAnimationCurveType InType, EDatasmithFBXSceneAnimationCurveComponent InComponent)
{
	if (InCurve == nullptr)
	{
		UE_LOGF(LogDatasmithFBXImport, Warning, "InCurve was nullptr!");
		return;
	}

	int32 DSID = -1.0f;
	FDatasmithFBXSceneAnimCurve* Curve = nullptr;

	bool bParsedTagTime = OutScene->TagTime != FLT_MAX;
	int KeyCount = InCurve->KeyGetCount();
	if (KeyCount < 1)
	{
		return;
	}

	// First key will be the DSID tag, and it's time will tell us the DSID
	// With the DSID, we can check the .clips file and discover the actual StartTime of the curve
	FbxAnimCurveKey FirstKey = InCurve->KeyGet(0);
	float FirstKeyTime = SAFE_FLT((float)FirstKey.GetTime().GetSecondDouble());

	// All first keys are before or at TagTime, and all real StartSeconds are 1 second after TagTime
	if (FirstKeyTime > OutScene->TagTime)
	{
		UE_LOGF(LogDatasmithFBXImport, Warning, "Could not find AnimCurve for DSID '%d' parsed at '%f'", DSID, FirstKeyTime);
		return;
	}

	DSID = FMath::RoundToInt((OutScene->TagTime - FirstKeyTime) * OutScene->BaseTime);

	if (TArray<FDatasmithFBXSceneAnimCurve*>* FoundCurves = ImportedAnimationCurves.Find(DSID))
	{
		for (FDatasmithFBXSceneAnimCurve* FoundCurve : *FoundCurves)
		{
			if (FoundCurve->Type == InType && FoundCurve->Component == InComponent)
			{
				UE_LOGF(LogDatasmithFBXImport, Verbose, "Matched to curve with DSID %d", DSID);
				Curve = FoundCurve;
				break;
			}
		}
	}

	if (Curve == nullptr)
	{
		UE_LOGF(LogDatasmithFBXImport, Warning, "Failed to match AnimCurve of type %u and component %u to the imported data from .clips file", (uint8)InType, (uint8)InComponent);
		return;
	}

	// Prune the FBX curve from potential keypoints added by VRED if it tried to bake the animation
	// curves (happens when we have rotation orientation on the node or animations)
	float RealStartSeconds = Curve->StartTimeSeconds;
	int32 LastIndexToPrune = 0;
	for (int32 KeyIndex = 0; KeyIndex < KeyCount; KeyIndex++)
	{
		FbxAnimCurveKey Key = InCurve->KeyGet(KeyIndex);
		float KeyTime = SAFE_FLT((float)Key.GetTime().GetSecondDouble());

		if(KeyTime < RealStartSeconds)
		{
			LastIndexToPrune = KeyIndex;
		}
	}
	InCurve->KeyRemove(0, LastIndexToPrune);

	ProcessCurve(InCurve, Curve, InType, InComponent);
}

FDatasmithFBXSceneAnimCurve FDatasmithFBXFileImporter::CreateNewCurve(FbxAnimCurve* InCurve, EDatasmithFBXSceneAnimationCurveType InType, EDatasmithFBXSceneAnimationCurveComponent InComponent)
{
	FDatasmithFBXSceneAnimCurve Curve;
	Curve.Type = InType;
	Curve.Component = InComponent;

	int KeyCount = InCurve->KeyGetCount();
	if (KeyCount == 1)
	{
		FbxAnimCurveKey FirstKey = InCurve->KeyGet(0);
		float Val = FirstKey.GetValue();
		if (FMath::IsNearlyEqual(Val, 0.0f))
		{
			InCurve->KeyRemove(0);

			if (!bDisplayedTwoKeysWarning)
			{
				UE_LOGF(LogDatasmithFBXImport, Warning, "Discarding animation curves for having a single key with value zero, as they are likely artifacts of not having .clips data. You can provide two keys with value zero if you wish to prevent this behaviour.");
				bDisplayedTwoKeysWarning = true;
			}

			return Curve;
		}
	}

	ProcessCurve(InCurve, &Curve, InType, InComponent);

	return Curve;
}

/**
* Tries to extract animation curves of InProperty, building a new FDatasmithFBXSceneAnimCurve of type InType for each channel, if it can
*/
void FDatasmithFBXFileImporter::AddCurvesForProperty(FbxProperty InProperty, FbxAnimLayer* InLayer, EDatasmithFBXSceneAnimationCurveType Type, TArray<FDatasmithFBXSceneAnimCurve>& OutCurves)
{
	if (InProperty.IsValid())
	{
		auto PropertyName = StringCast<ANSICHAR>(InProperty.GetNameAsCStr());

		FbxAnimCurveNode* CurveNode = InProperty.GetCurveNode(InLayer);

		if (CurveNode != nullptr)
		{
			uint32 ChannelsCount = CurveNode->GetChannelsCount();
			for (uint8 Channel = 0; Channel < CurveNode->GetChannelsCount(); Channel++)
			{
				uint32 CurveCount = CurveNode->GetCurveCount(Channel);
				UE_LOGF(LogDatasmithFBXImport, Verbose, "\tFound %d curves for property %s, channel %d", CurveCount, PropertyName.Get(), Channel);

				for (uint32 CurveIndex = 0; CurveIndex < CurveCount; CurveIndex++)
				{
					FbxAnimCurve* FbxCurve = CurveNode->GetCurve(Channel, CurveIndex);
					if (FbxCurve != nullptr)
					{
						// If we have parsed the clips file, we'll have a TagTime. Then, we already have a valid curve
						// in OutScene->AnimNodes that corresponds to it, so we just have to find it and fill it in.
						// If we have not parsed it (the user didn't chose one or something failed), we'll just create a new
						// curve right now and add it to this OutCurves object, where later we will try our best to separate them
						// into AnimBlocks
						if (FMath::IsNearlyEqual(OutScene->TagTime, FLT_MAX))
						{
							OutCurves.Add(CreateNewCurve(FbxCurve, Type, (EDatasmithFBXSceneAnimationCurveComponent)Channel));
						}
						else
						{
							FillCurveFromClipsFile(FbxCurve, Type, (EDatasmithFBXSceneAnimationCurveComponent)Channel);
						}
					}
				}
			}
		}
	}
}

void FDatasmithFBXFileImporter::ExtractAnimations(FbxNode* Node)
{
	FDatasmithFBXSceneAnimNode* Animation = new(OutScene->AnimNodes) FDatasmithFBXSceneAnimNode;
	Animation->Name = FString(ANSI_TO_TCHAR(Node->GetName()));

	FDatasmithFBXSceneAnimBlock* AllCurves = new(Animation->Blocks) FDatasmithFBXSceneAnimBlock;
	AllCurves->Name = Animation->Name + FString(TEXT("Block_0"));

	UE_LOGF(LogDatasmithFBXImport, Verbose, "Extracting animation curves for node %ls", UTF8_TO_TCHAR(Node->GetName()));

	// Iterate over animation stacks and layers, but basically ignore that structure since VRED always emits everything
	// into AnimStack1/Layer0 anyway
	int32 AnimStackCount = InScene->GetSrcObjectCount<FbxAnimStack>();
	for (int32 AnimStackIndex = 0; AnimStackIndex < AnimStackCount; AnimStackIndex++)
	{
		FbxAnimStack* CurAnimStack = InScene->GetSrcObject<FbxAnimStack>(AnimStackIndex);
		int32 AnimLayerCount = CurAnimStack->GetMemberCount();
		for (int32 AnimLayerIndex = 0; AnimLayerIndex < AnimLayerCount; AnimLayerIndex++)
		{
			FbxAnimLayer* CurAnimLayer = CurAnimStack->GetSrcObject<FbxAnimLayer>(AnimLayerIndex);

			AddCurvesForProperty(Node->LclTranslation, CurAnimLayer, EDatasmithFBXSceneAnimationCurveType::Translation, AllCurves->Curves);
			AddCurvesForProperty(Node->LclRotation, CurAnimLayer, EDatasmithFBXSceneAnimationCurveType::Rotation, AllCurves->Curves);
			AddCurvesForProperty(Node->LclScaling, CurAnimLayer, EDatasmithFBXSceneAnimationCurveType::Scale, AllCurves->Curves);
			AddCurvesForProperty(Node->Visibility, CurAnimLayer, EDatasmithFBXSceneAnimationCurveType::Visible, AllCurves->Curves);
		}
	}

	// Either we didn't find any curves or we just added them to OutScene directly (we have a clips file)
	if (AllCurves->Curves.Num() == 0)
	{
		OutScene->AnimNodes.Pop();
		return;
	}

	// IDs are attributed sequentially, so curves with sequential IDs are more likely to belong to the same block
	AllCurves->Curves.Sort();

	// Catalogue all encountered curves, so that we can easily tell how many blocks we'll need
	TMap<uint16, TArray<const FDatasmithFBXSceneAnimCurve*>> TypeChannelToCurves;
	for (const FDatasmithFBXSceneAnimCurve& Curve : AllCurves->Curves)
	{
		auto& CurvesOfTypeAndChannel = TypeChannelToCurves.FindOrAdd((uint16)((uint8)Curve.Type << 8 | (uint8)Curve.Component));
		CurvesOfTypeAndChannel.Add(&Curve);
	}

	// We can have at most one curve of each type/channel combination per block
	int32 NumNeededBlocks = 0;
	for (auto& Elem : TypeChannelToCurves)
	{
		NumNeededBlocks = FMath::Max(NumNeededBlocks, Elem.Value.Num());
	}

	// It might be that this is all a waste since we'll process these blocks better within a specific module's importer, but
	// we can't know that yet, so let's split these curves as best as we can
	UE_LOGF(LogDatasmithFBXImport, Verbose, "Need %d animation blocks to fit %d curves", NumNeededBlocks, AllCurves->Curves.Num());
	if (NumNeededBlocks > 1)
	{
		// Go over TypeChannelToCurves building as many blocks as we need,
		// spreading at most one of each type/channel combination into each block
		for (int32 NewBlockIndex = 0; NewBlockIndex < NumNeededBlocks; NewBlockIndex++)
		{
			FDatasmithFBXSceneAnimBlock* NewBlock = new(Animation->Blocks) FDatasmithFBXSceneAnimBlock;
			NewBlock->Name = Animation->Name + FString(TEXT("Block_")) + FString::FromInt(NewBlockIndex);

			for (auto& Elem : TypeChannelToCurves)
			{
				auto& Curves = Elem.Value;

				if (Curves.Num() > 0)
				{
					NewBlock->Curves.Add(*Curves[0]);

					Curves.RemoveAt(0);
				}
			}
		}

		Animation->Blocks.RemoveAt(0);
	}
}

struct FFbxUVInfo
{
	FString Name;
	const FbxLayerElementUV* LayerElementUV;
	FbxLayerElement::EReferenceMode UVReferenceMode;
	FbxLayerElement::EMappingMode UVMappingMode;

	bool operator<(const FFbxUVInfo& Other) const
	{
		return Name < Other.Name;
	}
};

void FDatasmithFBXFileImporter::FindFbxUVChannels(FbxMesh* Mesh, TArray<FFbxUVInfo>& FbxUVs)
{
	int32 NumLayers = Mesh->GetLayerCount();
	for (int32 LayerIndex = 0; LayerIndex < NumLayers; LayerIndex++)
	{
		const FbxLayer* Layer = Mesh->GetLayer(LayerIndex);
		int NumUVLayers = Layer->GetUVSetCount();
		if (NumUVLayers)
		{
			FbxArray<const FbxLayerElementUV*> LayerUVs = Layer->GetUVSets();
			FbxArray<FbxLayerElement::EType> LayerTypes = Layer->GetUVSetChannels();

			for (int32 LayerUVIndex = 0; LayerUVIndex < NumUVLayers; LayerUVIndex++)
			{
				const FbxLayerElementUV* UvLayer = LayerUVs[LayerUVIndex];
				FString UvName = UvLayer->GetName();
				bool bAlreadyExists = false;
				for (int32 i = 0; i < FbxUVs.Num(); i++)
				{
					if (FbxUVs[i].Name == UvName)
					{
						bAlreadyExists = true;
						break;
					}
				}
				if (!bAlreadyExists)
				{
					FFbxUVInfo* UV = new(FbxUVs) FFbxUVInfo;
					UV->Name = UvName;
					UV->LayerElementUV = UvLayer;
					UV->UVReferenceMode = UvLayer->GetReferenceMode();
					UV->UVMappingMode = UvLayer->GetMappingMode();
				}
			}
		}
	}

	// Sort UVs by name, so "DiffuseUV" goes before "ShadowUV"
	if (FbxUVs.Num() > 1)
	{
		FbxUVs.Sort();
	}
}

TSharedPtr<FDatasmithFBXSceneMesh> FDatasmithFBXFileImporter::ImportMesh(FbxMesh* InMesh, FbxNode* InNode)
{
	// Check if this mesh was already imported
	TSharedPtr<FDatasmithFBXSceneMesh>* ImportedMesh = ImportedMeshes.Find(InMesh);
	if (ImportedMesh != nullptr)
	{
		return *ImportedMesh;
	}

	// Create a new mesh
	TSharedPtr<FDatasmithFBXSceneMesh> Mesh(new FDatasmithFBXSceneMesh());
	ImportedMeshes.Add(InMesh, Mesh);

	Mesh->Name = FDatasmithUtils::SanitizeObjectName(UTF8_TO_TCHAR(InMesh->GetName()));
	if (Mesh->Name.IsEmpty())
	{
		// Use node's name as mesh name if mesh name is empty
		Mesh->Name = FDatasmithUtils::SanitizeObjectName(UTF8_TO_TCHAR(InNode->GetName()));
	}

	int32 Pos = Mesh->Name.Find(TEXT(NAMECLASH1_KEY));
	if (Pos != INDEX_NONE)
	{
		Mesh->Name = Mesh->Name.Left(Pos);
	}

	Mesh->ImportMaterialCount = InNode->GetMaterialCount();

	return Mesh;
}


void FDatasmithFBXFileImporter::DoImportMesh(FbxMesh* InMesh, FDatasmithFBXSceneMesh* Mesh)
{
	// Reference code: UnFbx::FFbxImporter::BuildStaticMeshFromGeometry()
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithFBXFileImporter::DoImportMesh)

	int32 MaterialCount = FMath::Max(Mesh->ImportMaterialCount, 1);

	FbxLayer* BaseLayer = InMesh->GetLayer(0);
	if (BaseLayer == nullptr)
	{
		UE_LOGF(LogDatasmithFBXImport, Error, "There is no geometry information in mesh '%ls'", *Mesh->Name);
		return;
	}

	int32 PolygonCount = InMesh->GetPolygonCount();
	int32 VertexCount = InMesh->GetControlPointsCount();
	if (PolygonCount == 0)
	{
		UE_LOGF(LogDatasmithFBXImport, Warning, "No Polygon in mesh '%ls'", *Mesh->Name);
		return;
	}

	// Get the "material index" layer.  Do this AFTER the triangulation step as that may reorder material indices
	FbxLayerElementMaterial* LayerElementMaterial = BaseLayer->GetMaterials();
	FbxLayerElement::EMappingMode MaterialMappingMode = LayerElementMaterial ? LayerElementMaterial->GetMappingMode() : FbxLayerElement::EMappingMode::eNone;

	// Get the smoothing group layer
	FbxLayerElementSmoothing* SmoothingInfo = BaseLayer->GetSmoothing();
	FbxLayerElement::EReferenceMode SmoothingReferenceMode = SmoothingInfo ? SmoothingInfo->GetReferenceMode() : FbxLayerElement::EReferenceMode::eDirect;
	FbxLayerElement::EMappingMode SmoothingMappingMode = SmoothingInfo ? SmoothingInfo->GetMappingMode() : FbxLayerElement::EMappingMode::eNone;

	if (SmoothingMappingMode == FbxLayerElement::EMappingMode::eByPolygon)
	{
		FbxGeometryConverter* GeometryConverter = UnFbx::FFbxImporter::GetInstance()->GetGeometryConverter();
		if (ensure(GeometryConverter))
		{
			GeometryConverter->ComputeEdgeSmoothingFromPolygonSmoothing(InMesh);
			// update SmoothingInfo
			SmoothingInfo = BaseLayer->GetSmoothing();
			SmoothingReferenceMode = SmoothingInfo ? SmoothingInfo->GetReferenceMode() : FbxLayerElement::EReferenceMode::eDirect;
			SmoothingMappingMode = SmoothingInfo ? SmoothingInfo->GetMappingMode() : FbxLayerElement::EMappingMode::eNone;
		}
	}

	if (SmoothingMappingMode != FbxLayerElement::EMappingMode::eByEdge && SmoothingMappingMode != FbxLayerElement::EMappingMode::eNone)
	{
		UE_LOGF(LogDatasmithFBXImport, Warning, "Unsupported Smoothing group mapping mode on mesh '%ls'", *Mesh->Name);
	}

	// Get the first vertex color layer
	FbxLayerElementVertexColor* LayerElementVertexColor = BaseLayer->GetVertexColors();
	FbxLayerElement::EReferenceMode VertexColorReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode VertexColorMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementVertexColor)
	{
		VertexColorReferenceMode = LayerElementVertexColor->GetReferenceMode();
		VertexColorMappingMode = LayerElementVertexColor->GetMappingMode();
	}

	// Get the first normal layer
	FbxLayerElementNormal* LayerElementNormal = BaseLayer->GetNormals();
	FbxLayerElementTangent* LayerElementTangent = BaseLayer->GetTangents();
	FbxLayerElementBinormal* LayerElementBinormal = BaseLayer->GetBinormals();

	// Whether there is normal, tangent and binormal data in this mesh
	bool bHasNTBInformation = LayerElementNormal && LayerElementTangent && LayerElementBinormal;

	FbxLayerElement::EReferenceMode NormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode NormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementNormal)
	{
		NormalReferenceMode = LayerElementNormal->GetReferenceMode();
		NormalMappingMode = LayerElementNormal->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode TangentReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode TangentMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementTangent)
	{
		TangentReferenceMode = LayerElementTangent->GetReferenceMode();
		TangentMappingMode = LayerElementTangent->GetMappingMode();
	}

	FbxLayerElement::EReferenceMode BinormalReferenceMode(FbxLayerElement::eDirect);
	FbxLayerElement::EMappingMode BinormalMappingMode(FbxLayerElement::eByControlPoint);
	if (LayerElementBinormal)
	{
		BinormalReferenceMode = LayerElementBinormal->GetReferenceMode();
		BinormalMappingMode = LayerElementBinormal->GetMappingMode();
	}

	bool OddNegativeScale = false; // IsOddNegativeScale(TotalMatrix);
	// offsets are useful to append data in an existing mesh
	int32 VertexOffset = 0;         // MeshDescription.Vertices().Num();
	int32 VertexInstanceOffset = 0; // MeshDescription.VertexInstances().Num();
	int32 PolygonOffset = 0;        // MeshDescription.Polygons().Num();

	FMeshDescription MeshDescription;

	FStaticMeshAttributes StaticMeshAttributes{ MeshDescription };
	StaticMeshAttributes.Register();

	TVertexAttributesRef<FVector3f> VertexPositions = StaticMeshAttributes.GetVertexPositions();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = StaticMeshAttributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = StaticMeshAttributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = StaticMeshAttributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = StaticMeshAttributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = StaticMeshAttributes.GetVertexInstanceUVs();
	TEdgeAttributesRef<bool> EdgeHardnesses = StaticMeshAttributes.GetEdgeHardnesses();
	TPolygonGroupAttributesRef<FName> PolygonGroupImportedMaterialSlotNames = StaticMeshAttributes.GetPolygonGroupMaterialSlotNames();

	// Reserve space for attributes.
	MeshDescription.ReserveNewVertices(VertexCount);
	MeshDescription.ReserveNewVertexInstances(InMesh->GetPolygonVertexCount());
	MeshDescription.ReserveNewEdges(InMesh->GetMeshEdgeCount());
	MeshDescription.ReserveNewPolygons(PolygonCount);
	MeshDescription.ReserveNewPolygonGroups(MaterialCount);

	TArray<FFbxUVInfo> FBXUVs;
	FindFbxUVChannels(InMesh, FBXUVs);
	int32 FbxUVCount = FBXUVs.Num();

	// At least one UV set must exist.
	int32 MeshDescUVCount = FMath::Max(1, FbxUVCount);
	VertexInstanceUVs.SetNumChannels(MeshDescUVCount);

	//Fill the vertex array
	for (int32 VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	{
		FVertexID AddedVertexId = MeshDescription.CreateVertex();
		check(AddedVertexId.GetValue() == VertexIndex);

		FbxVector4 FbxPosition = InMesh->GetControlPoints()[VertexIndex];
		const FVector VertexPosition = FBXFileImporterImpl::ConvertPos(FbxPosition);
		VertexPositions[AddedVertexId] = (FVector3f)VertexPosition;
	}

	TArray<int32> ValidFbxUVLayerIndices;
	ValidFbxUVLayerIndices.Reserve(MeshDescUVCount);
	for (int32 UVLayerIndex = 0; UVLayerIndex < MeshDescUVCount; UVLayerIndex++)
	{
		if (FBXUVs.IsValidIndex(UVLayerIndex) && FBXUVs[UVLayerIndex].LayerElementUV != nullptr)
		{
			ValidFbxUVLayerIndices.Add(UVLayerIndex);
		}
	}

	TMap<int32, FPolygonGroupID> PolygonGroupMapping;
	auto GetOrCreatePolygonGroupId = [&](int32 MaterialIndex)
	{
		FPolygonGroupID& PolyGroupId = PolygonGroupMapping.FindOrAdd(MaterialIndex);
		if (PolyGroupId == INDEX_NONE)
		{
			PolyGroupId = MeshDescription.CreatePolygonGroup();
			FName ImportedSlotName = DatasmithMeshHelper::DefaultSlotName(MaterialIndex);
			PolygonGroupImportedMaterialSlotNames[PolyGroupId] = ImportedSlotName;
		}
		return PolyGroupId;
	};

	// corner informations
	TArray<FVector> CornerPositions;
	TArray<FVertexInstanceID> CornerVertexInstanceIDs;
	TArray<FVertexID> CornerVertexIDs;
	int32 FirstFbxInstanceIndexForPoly = 0;
	int32 NextPolyFbxInstanceIndex = 0;
	for (int32 PolygonIndex = 0; PolygonIndex < PolygonCount; PolygonIndex++)
	{
		// collect corners positions
		int32 CornerCount = InMesh->GetPolygonSize(PolygonIndex);
		FirstFbxInstanceIndexForPoly = NextPolyFbxInstanceIndex;
		NextPolyFbxInstanceIndex += CornerCount;
		if (! ensure(CornerCount >= 3))
		{
			continue;
		}

		CornerPositions.SetNumUninitialized(CornerCount, EAllowShrinking::No);
		CornerVertexIDs.SetNumUninitialized(CornerCount, EAllowShrinking::No);
		for (int32 CornerIndex = 0; CornerIndex < CornerCount; CornerIndex++)
		{
			const int32 ControlPointIndex = InMesh->GetPolygonVertex(PolygonIndex, CornerIndex);
			CornerVertexIDs[CornerIndex] = FVertexID(ControlPointIndex);
			CornerPositions[CornerIndex] = (FVector)VertexPositions[CornerVertexIDs[CornerIndex]];
		}

		// Skip degenerated polygons
		/*FVector RawNormal = ((CornerPositions[1] - CornerPositions[2]) ^ (CornerPositions[0] - CornerPositions[2]));
		if (RawNormal.SizeSquared() < SMALL_NUMBER)
		{
			continue; // this will leave holes...
		}*/

		// Create Vertex instances
		CornerVertexInstanceIDs.SetNumUninitialized(CornerCount, EAllowShrinking::No);
		for (int32 CornerIndex = 0; CornerIndex < CornerCount; CornerIndex++)
		{
			CornerVertexInstanceIDs[CornerIndex] = MeshDescription.CreateVertexInstance(CornerVertexIDs[CornerIndex]);
		}

		// UVs attributes
		for (int32 CornerIndex = 0; CornerIndex < CornerCount; CornerIndex++)
		{
			for (int32 UVLayerIndex : ValidFbxUVLayerIndices)
			{
				int32 UVMapIndex = (FBXUVs[UVLayerIndex].UVMappingMode == FbxLayerElement::eByControlPoint) ? CornerVertexIDs[CornerIndex].GetValue() : FirstFbxInstanceIndexForPoly + CornerIndex;
				int32 UVIndex = (FBXUVs[UVLayerIndex].UVReferenceMode == FbxLayerElement::eDirect) ? UVMapIndex : FBXUVs[UVLayerIndex].LayerElementUV->GetIndexArray().GetAt(UVMapIndex);
				FbxVector2 UVVector = FBXUVs[UVLayerIndex].LayerElementUV->GetDirectArray().GetAt(UVIndex);
				FVector2f FinalUVVector(static_cast<float>(UVVector[0]), 1.f - static_cast<float>(UVVector[1])); // flip the Y of UVs for DirectX
				if (!FinalUVVector.ContainsNaN())
				{
					VertexInstanceUVs.Set(CornerVertexInstanceIDs[CornerIndex], UVLayerIndex, FinalUVVector);
				}
			}
		}

		// Vertex colors
		if (LayerElementVertexColor)
		{
			for (int32 CornerIndex = 0; CornerIndex < CornerCount; CornerIndex++)
			{
				// if (VertexColorImportOption == EVertexColorImportOption::Replace)
				int32 VertexColorMappingIndex = (VertexColorMappingMode == FbxLayerElement::eByControlPoint) ? CornerVertexIDs[CornerIndex].GetValue() : FirstFbxInstanceIndexForPoly + CornerIndex;
				int32 VectorColorIndex = (VertexColorReferenceMode == FbxLayerElement::eDirect) ? VertexColorMappingIndex : LayerElementVertexColor->GetIndexArray().GetAt(VertexColorMappingIndex);
				FbxColor VertexColor = LayerElementVertexColor->GetDirectArray().GetAt(VectorColorIndex);
				// (no reason to convert to linear color as source is not sRGB)
				FVector4f RawColor(float(VertexColor.mRed), float(VertexColor.mGreen), float(VertexColor.mBlue), float(VertexColor.mAlpha));
				VertexInstanceColors[CornerVertexInstanceIDs[CornerIndex]] = RawColor;
			}
		}

		// Normals, tangents and binormals
		if (LayerElementNormal)
		{
			for (int32 CornerIndex = 0; CornerIndex < CornerCount; CornerIndex++)
			{
				FVertexInstanceID InstanceID = CornerVertexInstanceIDs[CornerIndex];
				int32 FbxInstanceIndex = FirstFbxInstanceIndexForPoly + CornerIndex;
				int32 VertexIndex = CornerVertexIDs[CornerIndex].GetValue();

				// normals may have different reference and mapping mode than tangents and binormals
				int NormalMapIndex = (NormalMappingMode == FbxLayerElement::eByControlPoint) ? VertexIndex : FbxInstanceIndex;
				int NormalValueIndex = (NormalReferenceMode == FbxLayerElement::eDirect) ? NormalMapIndex : LayerElementNormal->GetIndexArray().GetAt(NormalMapIndex);
				FbxVector4 FbxTangentZ = LayerElementNormal->GetDirectArray().GetAt(NormalValueIndex);
				const FVector TangentZ = FBXFileImporterImpl::ConvertDir(FbxTangentZ);
				VertexInstanceNormals[InstanceID] = (FVector3f)TangentZ.GetSafeNormal();

				//tangents and binormals share the same reference, mapping mode and index array
				if (bHasNTBInformation)
				{
					int TangentMapIndex = (TangentMappingMode == FbxLayerElement::eByControlPoint) ? VertexIndex : FbxInstanceIndex;
					int TangentValueIndex = (TangentReferenceMode == FbxLayerElement::eDirect) ? TangentMapIndex : LayerElementTangent->GetIndexArray().GetAt(TangentMapIndex);
					FbxVector4 FbxTangentX = LayerElementTangent->GetDirectArray().GetAt(TangentValueIndex);
					FVector TangentX = FBXFileImporterImpl::ConvertDir(FbxTangentX);
					VertexInstanceTangents[InstanceID] = (FVector3f)TangentX.GetSafeNormal();

					int BinormalMapIndex = (BinormalMappingMode == FbxLayerElement::eByControlPoint) ? VertexIndex : FbxInstanceIndex;
					int BinormalValueIndex = (BinormalReferenceMode == FbxLayerElement::eDirect) ? BinormalMapIndex : LayerElementBinormal->GetIndexArray().GetAt(BinormalMapIndex);
					FbxVector4 FbxTangentY = LayerElementBinormal->GetDirectArray().GetAt(BinormalValueIndex);
					FVector TangentY = -FBXFileImporterImpl::ConvertDir(FbxTangentY);

					float BinormalSign = ((TangentZ ^ TangentX) | TangentY) < 0 ? -1.0f : 1.0f;
					check(BinormalSign == GetBasisDeterminantSign(TangentX.GetSafeNormal(), TangentY.GetSafeNormal(), TangentZ.GetSafeNormal()));
					VertexInstanceBinormalSigns[InstanceID] = BinormalSign;
				}
			}
		}

		// Material index
		int32 MaterialIndex = 0;
		if (MaterialCount > 0 && LayerElementMaterial != nullptr)
		{
			switch (MaterialMappingMode)
			{
				// material index is stored in the IndexArray, not the DirectArray (which is irrelevant with 2009.1)
				case FbxLayerElement::eAllSame:
				{
					MaterialIndex = LayerElementMaterial->GetIndexArray().GetAt(0);
					break;
				}
				case FbxLayerElement::eByPolygon:
				{
					MaterialIndex = LayerElementMaterial->GetIndexArray().GetAt(PolygonIndex);
					break;
				}
			}
		}

		// Create polygon outer loop
		for (int32 CornerIndex = 0; CornerIndex < CornerCount; ++CornerIndex)
		{
			uint32 NextCornerIndex = (CornerIndex + 1) % CornerCount;

			// Get or create edge
			FEdgeID EdgeId = MeshDescription.GetVertexPairEdge(CornerVertexIDs[CornerIndex], CornerVertexIDs[NextCornerIndex]);
			if (EdgeId == INDEX_NONE)
			{
				EdgeId = MeshDescription.CreateEdge(CornerVertexIDs[CornerIndex], CornerVertexIDs[NextCornerIndex]);

				// Crease sharpness
				int32 FbxEdgeIndex = InMesh->GetMeshEdgeIndexForPolygon(PolygonIndex, CornerIndex);

				// Smoothing
				bool Hardness = true;
				if (SmoothingMappingMode == FbxLayerElement::eByEdge)
				{
					int32 lSmoothingIndex = (SmoothingReferenceMode == FbxLayerElement::eDirect) ? FbxEdgeIndex : SmoothingInfo->GetIndexArray().GetAt(FbxEdgeIndex);
					Hardness = (SmoothingInfo->GetDirectArray().GetAt(lSmoothingIndex) == 0);
				}
				EdgeHardnesses[EdgeId] = Hardness;
			}
		}

		// Create in-mesh Polygon
		const FPolygonGroupID PolygonGroupID = GetOrCreatePolygonGroupId(MaterialIndex);
		const FPolygonID NewPolygonID = MeshDescription.CreatePolygon(PolygonGroupID, CornerVertexInstanceIDs);
	} // for poly loop

	Mesh->MeshDescription = MoveTemp(MeshDescription);
}

void FetchFbxProperty(FbxSurfaceMaterial* InMaterial, const FString& FbxPropertyName, FDatasmithFBXSceneMaterial& InOutMat)
{
	FbxProperty Prop = InMaterial->FindProperty(TCHAR_TO_UTF8(*FbxPropertyName));
	switch (Prop.GetPropertyDataType().GetType())
	{
	case eFbxFloat:
	{
		float& Value = InOutMat.ScalarParams.FindOrAdd(FbxPropertyName);
		Value = Prop.Get<FbxFloat>();
		break;
	}
	case eFbxDouble:
	{
		float& Value = InOutMat.ScalarParams.FindOrAdd(FbxPropertyName);
		Value = Prop.Get<FbxDouble>();
		break;
	}
	case eFbxDouble3:
	{
		FVector4& Value = InOutMat.VectorParams.FindOrAdd(FbxPropertyName);
		FbxVector4 v = Prop.Get<FbxVector4>();
		Value.X = v.mData[0];
		Value.Y = v.mData[1];
		Value.Z = v.mData[2];
		Value.W = 1.0f;
		break;
	}
	case eFbxDouble4:
	{
		FVector4& Value = InOutMat.VectorParams.FindOrAdd(FbxPropertyName);
		FbxVector4 v = Prop.Get<FbxVector4>();
		Value.X = v.mData[0];
		Value.Y = v.mData[1];
		Value.Z = v.mData[2];
		Value.W = v.mData[3];
		break;
	}
	default:
		return;
	}
}

void FetchFbxTexture(FbxSurfaceMaterial* InMaterial, const FString& InFbxPropertyName, FDatasmithFBXSceneMaterial& InOutMat, const FString& TextureDir)
{
	FbxProperty Prop = InMaterial->FindProperty(TCHAR_TO_UTF8(*InFbxPropertyName));
	if (Prop.GetSrcObjectCount<FbxTexture>() > 0)
	{
		FbxTexture* FbxTex = FbxCast<FbxTexture>(Prop.GetSrcObject<FbxTexture>(0));
		FString Path = UTF8_TO_TCHAR(FbxTex->GetName());
		if (Path.IsEmpty())
		{
			return;
		}

		Path = FPaths::GetCleanFilename(Path);
		Path = FPaths::Combine(TextureDir, Path);

		if (!FPaths::FileExists(Path))
		{
			return;
		}

		FString TextureName = InFbxPropertyName;
		TextureName.RemoveFromEnd(TEXT("Color"));	// DiffuseColor -> Diffuse
		TextureName.RemoveFromEnd(TEXT("Map"));	    // NormalMap -> Normal
		TextureName = TEXT("Tex") + TextureName;	// Diffuse -> TexDiffuse

		FDatasmithFBXSceneMaterial::FTextureParams& Tex = InOutMat.TextureParams.FindOrAdd(TextureName);
		Tex.Path = Path;

		FbxDouble3 v = FbxTex->Translation.Get();
		Tex.Translation.X = v[0];
		Tex.Translation.Y = v[1];
		Tex.Translation.Z = v[2];

		v = FbxTex->Rotation.Get();
		Tex.Rotation.X = v[0] * 1.f / 360.f;
		Tex.Rotation.Y = v[1] * 1.f / 360.f;
		Tex.Rotation.Z = v[2] * 1.f / 360.f;

		v = FbxTex->Scaling.Get();
		Tex.Scale.X = v[0];
		Tex.Scale.Y = v[1];
		Tex.Scale.Z = v[2];
	}
}

TSharedPtr<FDatasmithFBXSceneMaterial> FDatasmithFBXFileImporter::ImportMaterial(FbxSurfaceMaterial* InMaterial)
{
	// Check if this material was already imported
	TSharedPtr<FDatasmithFBXSceneMaterial>* ImportedMaterial = ImportedMaterials.Find(InMaterial);
	if (ImportedMaterial != nullptr)
	{
		return *ImportedMaterial;
	}

	// Create a new material
	TSharedPtr<FDatasmithFBXSceneMaterial> Material(new FDatasmithFBXSceneMaterial());
	ImportedMaterials.Add(InMaterial, Material);
	Material->Name = UTF8_TO_TCHAR(InMaterial->GetName());

	FetchFbxProperty(InMaterial, TEXT("EmissiveColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("EmissiveFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("AmbientColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("AmbientFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("DiffuseColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("DiffuseFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("BumpFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("TransparentColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("TransparencyFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("DisplacementColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("DisplacementFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("VectorDisplacementColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("VectorDisplacementFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("SpecularColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("SpecularFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("ReflectionColor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("ReflectionFactor"), *Material);
	FetchFbxProperty(InMaterial, TEXT("Shininess"), *Material);
	FetchFbxProperty(InMaterial, TEXT("ShininessExponent"), *Material);
	FetchFbxProperty(InMaterial, TEXT("Opacity"), *Material);
	FetchFbxProperty(InMaterial, TEXT("Reflectivity"), *Material);

	for (const FDirectoryPath& Path : Options->TextureDirs)
	{
		const FString& PathStr = Path.Path;
		if (!PathStr.IsEmpty() && FPaths::DirectoryExists(PathStr))
		{
			// These are the parameter names according to the FBX spec. Inside, we remove the suffixes
			// and add the Tex prefix
			FetchFbxTexture(InMaterial, TEXT("Bump"), *Material, PathStr);
			FetchFbxTexture(InMaterial, TEXT("NormalMap"), *Material, PathStr);
			FetchFbxTexture(InMaterial, TEXT("DiffuseColor"), *Material, PathStr);
			FetchFbxTexture(InMaterial, TEXT("SpecularColor"), *Material, PathStr);
			FetchFbxTexture(InMaterial, TEXT("ReflectionColor"), *Material, PathStr);
			FetchFbxTexture(InMaterial, TEXT("TransparentColor"), *Material, PathStr);
			FetchFbxTexture(InMaterial, TEXT("EmissiveColor"), *Material, PathStr);
			FetchFbxTexture(InMaterial, TEXT("Shininess"), *Material, PathStr);
		}
	}

	// Clamp strenght of emissive factors, as these can be arbitrarily large
	if (float* Emissive = Material->ScalarParams.Find(TEXT("EmissiveFactor")))
	{
		Material->ScalarParams[TEXT("EmissiveFactor")] = FMath::Min(*Emissive, 10.0f);
	}

	return Material;
}

// A small set of functions copy-pasted from UnrealEd due to inaccessibility at original location

bool FDatasmithFBXFileImporter::IsOddNegativeScale(FbxAMatrix& TotalMatrix)
{
	FbxVector4 Scale = TotalMatrix.GetS();
	int32 NegativeNum = 0;

	if (Scale[0] < 0) NegativeNum++;
	if (Scale[1] < 0) NegativeNum++;
	if (Scale[2] < 0) NegativeNum++;

	return NegativeNum == 1 || NegativeNum == 3;
}


