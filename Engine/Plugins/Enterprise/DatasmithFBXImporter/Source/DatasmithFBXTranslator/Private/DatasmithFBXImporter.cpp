// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFBXImporter.h"
#include "DatasmithFBXImporterLog.h"
#include "DatasmithFBXScene.h"
#include "IDatasmithSceneElements.h"


DEFINE_LOG_CATEGORY(LogDatasmithFBXImport);

namespace DatasmithFBXImporterImpl
{
	void BuildActorMapRecursive(TSharedPtr<IDatasmithActorElement> ActorElement, FActorMap& ActorMap)
	{
		if (!ActorElement.IsValid())
		{
			return;
		}

		FName OriginalName = ActorElement->GetTag(0);
		ActorMap.FindOrAdd(OriginalName).Add(ActorElement);

		for (int32 ChildIndex = 0; ChildIndex < ActorElement->GetChildrenCount(); ++ChildIndex)
		{
			BuildActorMapRecursive(ActorElement->GetChild(ChildIndex), ActorMap);
		}
	}
};

FDatasmithFBXImporter::FDatasmithFBXImporter()
{
	IntermediateScene = MakeUnique<FDatasmithFBXScene>();
}

FDatasmithFBXImporter::~FDatasmithFBXImporter()
{
}

void FDatasmithFBXImporter::GetGeometriesForMeshElementAndRelease(const TSharedRef<IDatasmithMeshElement> MeshElement, TArray<FMeshDescription>& OutMeshDescriptions)
{
	TSharedPtr<FDatasmithFBXSceneMesh>* FoundMesh = MeshNameToFBXMesh.Find(MeshElement->GetName());
	if (FoundMesh && (*FoundMesh).IsValid())
	{
		OutMeshDescriptions.Add(MoveTemp((*FoundMesh)->MeshDescription));
	}
	else
	{
		UE_LOGF(LogDatasmithFBXImport, Error, "Failed to return FMeshDescription object for requested mesh element '%ls'", MeshElement->GetName());
	}
}

void FDatasmithFBXImporter::BuildAssetMaps(TSharedRef<IDatasmithScene> Scene, FActorMap& ActorsByOriginalName, FMaterialMap& MaterialsByName)
{
	ActorsByOriginalName.Reset();
	for (int32 RootActorIndex = 0; RootActorIndex < Scene->GetActorsCount(); ++RootActorIndex)
	{
		DatasmithFBXImporterImpl::BuildActorMapRecursive(Scene->GetActor(RootActorIndex), ActorsByOriginalName);
	}

	MaterialsByName.Reset();
	for (const auto& MatEntry : ImportedMaterials)
	{
		const TSharedPtr<FDatasmithFBXSceneMaterial>& FBXMaterial = MatEntry.Key;
		TSharedPtr<IDatasmithBaseMaterialElement> DatasmithMaterial = MatEntry.Value;

		MaterialsByName.Add(FName(*FBXMaterial->Name), DatasmithMaterial);
	}
}

