// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFBXScene.h"

#include "DatasmithFBXHashUtils.h"
#include "DatasmithFBXImporterLog.h"
#include "Utility/DatasmithMeshHelper.h"

#include "StaticMeshAttributes.h"

int32 FDatasmithFBXSceneNode::NodeCounter = 1;

FDatasmithFBXSceneMaterial::FDatasmithFBXSceneMaterial()
{
}

FDatasmithFBXSceneMesh::FDatasmithFBXSceneMesh()
	: ImportMaterialCount(0)
	, bFlippedFaces(false)
{
}

FDatasmithFBXSceneMesh::~FDatasmithFBXSceneMesh()
{
}

const FMD5Hash& FDatasmithFBXSceneMesh::GetHash()
{
	if (!Hash.IsValid())
	{
		FMD5 Md5;
		DatasmithMeshHelper::HashMeshDescription(MeshDescription, Md5);
		Hash.Set(Md5);
	}
	return Hash;
}

bool FDatasmithFBXSceneMesh::HasNormals() const
{
	TVertexInstanceAttributesConstRef<FVector3f> Normals = FStaticMeshConstAttributes(MeshDescription).GetVertexInstanceNormals();
	return Normals.IsValid() && Normals.GetNumElements() > 0 && Normals[MeshDescription.VertexInstances().GetFirstValidID()].SizeSquared() > 0;
}

bool FDatasmithFBXSceneMesh::HasTangents() const
{
	TVertexInstanceAttributesConstRef<FVector3f> Tangents = FStaticMeshConstAttributes(MeshDescription).GetVertexInstanceTangents();
	return Tangents.IsValid() && Tangents.GetNumElements() > 0 && Tangents[MeshDescription.VertexInstances().GetFirstValidID()].SizeSquared() > 0;
}

FDatasmithFBXSceneCamera::FDatasmithFBXSceneCamera()
{
}

FDatasmithFBXSceneCamera::~FDatasmithFBXSceneCamera()
{
}

FDatasmithFBXSceneNode::FDatasmithFBXSceneNode()
	: Name(FString())
	, SplitNodeID(NodeCounter++)
	, Visibility(1.0f)
	, bVisibilityInheritance(false)
	, OriginalName(FString())
	, LocalTransform(FTransform::Identity)
	, RotationPivot(0.0f, 0.0f, 0.0f)
	, ScalingPivot(0.0f, 0.0f, 0.0f)
	, RotationOffset(0.0f, 0.0f, 0.0f)
	, ScalingOffset(0.0f, 0.0f, 0.0f)
	, bShouldKeepThisNode(false)
	, NodeType(ENodeType::Node)
{
}

FDatasmithFBXSceneNode::~FDatasmithFBXSceneNode()
{
}

FTransform FDatasmithFBXSceneNode::GetTransformRelativeToParent(TSharedPtr<FDatasmithFBXSceneNode>& InParent) const
{
	if (InParent.Get() == this)
	{
		return FTransform();
	}

	FTransform Transform = LocalTransform;
	for (TSharedPtr<FDatasmithFBXSceneNode> Node = Parent.Pin(); Node.IsValid() && Node != InParent; Node = Node->Parent.Pin())
	{
		FTransform B;
		FTransform::Multiply(&B, &Transform, &Node->LocalTransform);
		Transform = B;
	}
	return Transform;
}

FTransform FDatasmithFBXSceneNode::GetWorldTransform() const
{
	FTransform Transform = LocalTransform;
	for (TSharedPtr<FDatasmithFBXSceneNode> Node = Parent.Pin(); Node.IsValid(); Node = Node->Parent.Pin())
	{
		FTransform B;
		FTransform::Multiply(&B, &Transform, &Node->LocalTransform);
		Transform = B;
	}
	return Transform;
}

void FDatasmithFBXSceneNode::RemoveNode()
{
	check(Children.Num() == 0);

	// Unlink this node from its parent. This should initiate node destruction because it is holded
	// by TSharedPtr.

	TSharedPtr<FDatasmithFBXSceneNode> ParentNode = Parent.Pin();
	check(ParentNode.IsValid());

	for (int32 ChildIndex = 0; ChildIndex < ParentNode->Children.Num(); ChildIndex++)
	{
		if (ParentNode->Children[ChildIndex].Get() == this)
		{
			ParentNode->Children.RemoveAt(ChildIndex);
			return;
		}
	}

	// Should not get here
	UE_LOGF(LogDatasmithFBXImport, Fatal, "Unexpected behavior");
}

int32 FDatasmithFBXSceneNode::GetChildrenCountRecursive() const
{
	int32 Result = Children.Num();
	for (const TSharedPtr<FDatasmithFBXSceneNode>& Child : Children)
	{
		Result += Child->GetChildrenCountRecursive();
	}
	return Result;
}

void FDatasmithFBXSceneNode::KeepNode()
{
	bShouldKeepThisNode = true;
}

void FDatasmithFBXSceneNode::MarkSwitchNode()
{
	NodeType |= ENodeType::Switch;
	bShouldKeepThisNode = true;

	// Prevent the targets of SwitchObjects from being removed. This shouldn't be necessary,
	// but some scenes seem to have all mesh nodes named exactly the same (e.g shell_0).
	// In those cases, when we convert them to Switch variants, we will search for actors named
	// 'shell_0' for our switch options and likely end up picking the wrong actor
	for (int32 NodeIndex = 0; NodeIndex < Children.Num(); NodeIndex++)
	{
		Children[NodeIndex]->bShouldKeepThisNode = true;
	}
}

void FDatasmithFBXSceneNode::MarkToggleNode()
{
	NodeType |= ENodeType::Toggle;
	bShouldKeepThisNode = true;
}

void FDatasmithFBXSceneNode::ResetNodeType()
{
	if (NodeType != ENodeType::Node)
	{
		check(Children.Num() == 0);
		NodeType = ENodeType::Node;
	}
}

const FMD5Hash& FDatasmithFBXSceneNode::GetHash()
{
	if (!Hash.IsValid())
	{
		FMD5 Md5;

		if (bShouldKeepThisNode)
		{
			// For special nodes (switches, their children, etc), we should hash node names
			FDatasmithFBXHashUtils::UpdateHash(Md5, OriginalName);
		}

		if (Mesh.IsValid())
		{
			// Hash for geometry
			const FMD5Hash& MeshHash = Mesh->GetHash();
			FDatasmithFBXHashUtils::UpdateHash(Md5, MeshHash);

			// Hash for materials
			FDatasmithFBXHashUtils::UpdateHash(Md5, Materials.Num());
			for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); MaterialIndex++)
			{
				FDatasmithFBXHashUtils::UpdateHash(Md5, Materials[MaterialIndex]->Name);
			}
		}

		// Hash children
		FDatasmithFBXHashUtils::UpdateHash(Md5, Children.Num());
		// Sort children by hash to make hash invariant to children order
		TArray< TSharedPtr<FDatasmithFBXSceneNode> > SortedChildren = Children;
		SortedChildren.StableSort([](const TSharedPtr<FDatasmithFBXSceneNode>& A, const TSharedPtr<FDatasmithFBXSceneNode>& B)
			{
				return A->GetHash() < B->GetHash();
			});
		for (int32 ChildIndex = 0; ChildIndex < SortedChildren.Num(); ChildIndex++)
		{
			TSharedPtr<FDatasmithFBXSceneNode>& Node = SortedChildren[ChildIndex];
			// Use child hash
			FDatasmithFBXHashUtils::UpdateHash(Md5, Node->GetHash());
			// and its local transform related to this node
			FDatasmithFBXHashUtils::UpdateHash(Md5, Node->LocalTransform);
		}

		Hash.Set(Md5);
	}
	return Hash;
}

FDatasmithFBXScene::FDatasmithFBXScene()
	: TagTime(FLT_MAX)
	, ScaleFactor(1.0f)
{
}

FDatasmithFBXScene::~FDatasmithFBXScene()
{
}

TArray<TSharedPtr<FDatasmithFBXSceneNode>> FDatasmithFBXScene::GetAllNodes()
{
	TArray<TSharedPtr<FDatasmithFBXSceneNode>> Result;
	FDatasmithFBXSceneNode::Traverse(RootNode, [&Result](TSharedPtr<FDatasmithFBXSceneNode> Node)
	{
		Result.Add(Node);
	});
	return Result;
}

FDatasmithFBXScene::FStats FDatasmithFBXScene::GetStats()
{
	FStats Stats;

	MeshUseCountType CollectedMeshes;
	MaterialUseCountType CollectedMaterials;
	RecursiveCollectAllObjects(&CollectedMeshes, &CollectedMaterials, &Stats.NodeCount, RootNode);

	// Count all mesh instances in scene
	for (auto& MeshInfo : CollectedMeshes)
	{
		Stats.MeshCount += MeshInfo.Value;
	}

	Stats.GeometryCount = CollectedMeshes.Num();
	Stats.MaterialCount = CollectedMaterials.Num();
	return Stats;
}

void FDatasmithFBXScene::RecursiveCollectAllObjects(MeshUseCountType* Meshes, MaterialUseCountType* InMaterials, int32* NodeCount, const TSharedPtr<FDatasmithFBXSceneNode>& InNode) const
{
	FDatasmithFBXSceneNode::Traverse(InNode, [&](TSharedPtr<FDatasmithFBXSceneNode> Node)
	{
		if (NodeCount != nullptr)
		{
			// Count nodes
			(*NodeCount)++;
		}

		// Count meshes
		if (Node->Mesh.IsValid() && Meshes != nullptr)
		{
			int32* FoundMesh = Meshes->Find(Node->Mesh);
			if (FoundMesh == nullptr)
			{
				Meshes->Add(Node->Mesh, 1);
			}
			else
			{
				(*FoundMesh)++;
			}
		}

		// Count materials
		if (InMaterials != nullptr)
		{
			for (int32 MaterialIndex = 0; MaterialIndex < Node->Materials.Num(); MaterialIndex++)
			{
				TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[MaterialIndex];
				int32* FoundMaterial = InMaterials->Find(Material);
				if (FoundMaterial == nullptr)
				{
					InMaterials->Add(Material, 1);
				}
				else
				{
					(*FoundMaterial)++;
				}
			}
		}
	});
}

