// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFBXSceneProcessor.h"

#include "DatasmithFBXHashUtils.h"
#include "DatasmithFBXImporterLog.h"
#include "Misc/Paths.h"

#include "Misc/FileHelper.h"

#define ANIMNODE_SUFFIX						TEXT("_AnimNode")
#define LIGHT_SUFFIX						TEXT("_Light")
#define MESH_SUFFIX							TEXT("_Mesh")
#define CAMERA_SUFFIX						TEXT("_Camera")
#define MIN_TOTAL_NODES_TO_OPTIMIZE			30
#define MIN_NODES_IN_SUBTREE_TO_OPTIMIZE	5

FDatasmithFBXSceneProcessor::FDatasmithFBXSceneProcessor(FDatasmithFBXScene* InScene)
	: Scene(InScene)
{
}

struct FLightMapNodeRemover
{
	TMap< FString, TSharedPtr<FDatasmithFBXSceneMaterial> > NameToMaterial;

	void Recurse(TSharedPtr<FDatasmithFBXSceneNode>& Node)
	{
		bool IsLightMapMaterialPresent = false;
		bool IsOtherMaterialPresent = false;
		for (int32 MaterialIndex = 0; MaterialIndex < Node->Materials.Num(); MaterialIndex++)
		{
			TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[MaterialIndex];

			if (Material->Name.StartsWith(TEXT("Light_Map")))
			{
				IsLightMapMaterialPresent = true;
			}
			else
			{
				IsOtherMaterialPresent = true;
			}
		}

		for (int32 NodeIndex = 0; NodeIndex < Node->Children.Num(); NodeIndex++)
		{
			Recurse(Node->Children[NodeIndex]);
		}

		bool ShouldRemoveMesh = IsLightMapMaterialPresent && !IsOtherMaterialPresent;

		if (ShouldRemoveMesh)
		{
			Node->Mesh.Reset();
			Node->Materials.Empty();
		}
	}
};

void FDatasmithFBXSceneProcessor::RemoveLightMapNodes()
{
	FLightMapNodeRemover Finder;
	Finder.Recurse(Scene->RootNode);
}

struct FDupMaterialFinder
{
	TMap< FString, TSharedPtr<FDatasmithFBXSceneMaterial> > NameToMaterial;

	void Recurse(TSharedPtr<FDatasmithFBXSceneNode>& Node)
	{
		for (int32 MaterialIndex = 0; MaterialIndex < Node->Materials.Num(); MaterialIndex++)
		{
			TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[MaterialIndex];
			TSharedPtr<FDatasmithFBXSceneMaterial>* PrevMaterial = NameToMaterial.Find(Material->Name);
			if (PrevMaterial == nullptr)
			{
				// This is the first occurrence of this material name
				NameToMaterial.Add(Material->Name, Material);
			}
			else
			{
				// We already have a material with the same name, use that material
				Material = *PrevMaterial;
			}
		}

		for (int32 NodeIndex = 0; NodeIndex < Node->Children.Num(); NodeIndex++)
		{
			Recurse(Node->Children[NodeIndex]);
		}
	}
};

void FDatasmithFBXSceneProcessor::FindDuplicatedMaterials()
{
	FDupMaterialFinder Finder;
	Finder.Recurse(Scene->RootNode);
}

struct FDupMeshFinder
{
	TMap< FMD5Hash, TSharedPtr<FDatasmithFBXSceneMesh> > HashToMesh;

	void Recurse(TSharedPtr<FDatasmithFBXSceneNode>& Node)
	{
		if (Node->Mesh.IsValid())
		{
			const FMD5Hash& MeshHash = Node->Mesh->GetHash();
			TSharedPtr<FDatasmithFBXSceneMesh>* PrevMesh = HashToMesh.Find(MeshHash);
			if (PrevMesh != nullptr)
			{
				// We already have the same mesh, replace
				Node->Mesh = *PrevMesh;
			}
			else
			{
				HashToMesh.Add(MeshHash, Node->Mesh);
			}
		}

		for (int32 NodeIndex = 0; NodeIndex < Node->Children.Num(); NodeIndex++)
		{
			Recurse(Node->Children[NodeIndex]);
		}
	}
};

void FDatasmithFBXSceneProcessor::FindDuplicatedMeshes()
{
	FDupMeshFinder Finder;
	Finder.Recurse(Scene->RootNode);
}

void FDatasmithFBXSceneProcessor::RemoveEmptyNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node)
{
	// Recurse to children first. We're iterating in reverse order because iteration may change Children list.
	for (int32 NodeIndex = Node->Children.Num() - 1; NodeIndex >= 0; NodeIndex--)
	{
		RemoveEmptyNodesRecursive(Node->Children[NodeIndex]);
	}

	// Now check if we can remove this node
	if ( !( Node->bShouldKeepThisNode || Node->Children.Num() != 0 || Node->Mesh.IsValid() || !Node->Parent.IsValid() || Node->Camera.IsValid() || Node->Light.IsValid()) )
	{
		// This node doesn't have any children (probably they were removed with recursive call to this function),
		// and it wasn't marked as "read-only", so we can safely delete it.
		Node->RemoveNode();
	}
}

void FDatasmithFBXSceneProcessor::RemoveTempNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node)
{
	// Recurse to children first. We're iterating in reverse order because iteration may change Children list.
	for (int32 NodeIndex = Node->Children.Num() - 1; NodeIndex >= 0; NodeIndex--)
	{
		RemoveTempNodesRecursive(Node->Children[NodeIndex]);
	}

	// Now check if we can remove this node
	if (Node->OriginalName.MatchesWildcard(TEXT("__temp_*")))
	{
		Node->RemoveNode();
	}
}

struct FNodeMarkHelper
{
	TSet<FName> SwitchObjectNames;
	TSet<FName> ToggleObjectNames;
	TSet<FName> ObjectSetObjectNames;
	TSet<FName> AnimatedObjectNames;
	TSet<FName> SwitchMaterialObjectNames;
	TSet<FName> TransformVariantObjectNames;

	void Recurse(TSharedPtr<FDatasmithFBXSceneNode>& Node)
	{
		const FName NodeName = FName(*Node->OriginalName);
		if (NodeName != NAME_None)
		{
			if (SwitchObjectNames.Contains(NodeName))
			{
				Node->MarkSwitchNode();
			}
			if (ToggleObjectNames.Contains(NodeName))
			{
				Node->MarkToggleNode();
			}
			if (ObjectSetObjectNames.Contains(NodeName) ||
				AnimatedObjectNames.Contains(NodeName) ||
				SwitchMaterialObjectNames.Contains(NodeName) ||
				TransformVariantObjectNames.Contains(NodeName) ||
				Node->Light.IsValid() ||
				Node->Camera.IsValid())
			{
				Node->KeepNode();
			}
		}

		for (int32 NodeIndex = 0; NodeIndex < Node->Children.Num(); NodeIndex++)
		{
			Recurse(Node->Children[NodeIndex]);
		}
	}
};

void FDatasmithFBXSceneProcessor::FindPersistentNodes()
{
	FNodeMarkHelper Helper;
	Helper.SwitchObjectNames.Append(Scene->SwitchObjects);
	Helper.AnimatedObjectNames.Append(Scene->AnimatedObjects);
	Helper.SwitchMaterialObjectNames.Append(Scene->SwitchMaterialObjects);
	Helper.TransformVariantObjectNames.Append(Scene->TransformVariantObjects);
	Helper.ToggleObjectNames.Append(Scene->ToggleObjects);
	Helper.ObjectSetObjectNames.Append(Scene->ObjectSetObjects);

	Helper.Recurse(Scene->RootNode);
}

void FDatasmithFBXSceneProcessor::FixNodeNames()
{
	struct FFixNodeNamesHelper
	{
		void Do(FDatasmithFBXScene* InScene)
		{
			RecursiveFixNodeNames(InScene->RootNode);
		}

	protected:

		// this is copied from XmlFile.cpp to match their logic
		bool IsWhiteSpace(TCHAR Char)
		{
			// Whitespace will be any character that is not a common printed ASCII character (and not space/tab)
			if(Char == TCHAR(' ') ||
				Char == TCHAR('\t') ||
				Char < 32 ) // ' '
			{
				return true;
			}
			return false;
		}


		FString FixName(const FString& Name)
		{
			FString Result;
			bool LastWasWhitespace = false;
			for (int i = 0;i < Name.Len(); ++i)
			{
				TCHAR Char = Name[i];
				if (IsWhiteSpace(Char))
				{
					if (!LastWasWhitespace)
					{
						Result += TEXT(" ");
					}
					LastWasWhitespace = true;
				}
				else
				{
					Result += Char;
					LastWasWhitespace = false;
				}
			}
			return Result;
		}

		void RecursiveFixNodeNames(TSharedPtr<FDatasmithFBXSceneNode>& Node)
		{
			Node->Name = FixName(Node->Name);

			for (int32 NodeIndex = 0; NodeIndex < Node->Children.Num(); NodeIndex++)
			{
				TSharedPtr<FDatasmithFBXSceneNode>& Child = Node->Children[NodeIndex];

				RecursiveFixNodeNames(Child);
			}
		}
	};

	FFixNodeNamesHelper Helper;
	Helper.Do(Scene);
}

void FDatasmithFBXSceneProcessor::SplitLightNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node)
{
	// Recurse first so we don't check a potential separated child node
	for (TSharedPtr<FDatasmithFBXSceneNode> Child : Node->Children)
	{
		SplitLightNodesRecursive(Child);
	}

	if (Node->Light.IsValid())
	{
		TSharedPtr<FDatasmithFBXSceneNode> SeparatedChild = MakeShared<FDatasmithFBXSceneNode>();

		SeparatedChild->Name = Node->Name + FString(LIGHT_SUFFIX);
		SeparatedChild->OriginalName = Node->OriginalName;
		SeparatedChild->Light = Node->Light;
		SeparatedChild->SplitNodeID = Node->SplitNodeID;

		// Match light direction convention
		SeparatedChild->LocalTransform.SetIdentity();
		SeparatedChild->LocalTransform.ConcatenateRotation(FRotator(-90, 0, 0).Quaternion());

		/** Fix hierarchy

			P							P
			|							|
			N (light)		--->		N (node)
		   / \						  / | \
		  C1  C2					 C1 C2 SC (_Light node)

		P: Parent; N: Node; SC: SeparatedChild; C1,2: Children */
		SeparatedChild->Parent = Node;
		SeparatedChild->Children.Empty();
		Node->Children.Add(SeparatedChild);

		//Clean this Node
		Node->Light.Reset();
	}
}

void FDatasmithFBXSceneProcessor::SplitCameraNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node)
{
	// Recurse first so we don't check a potential separated child node
	for (TSharedPtr<FDatasmithFBXSceneNode> Child : Node->Children)
	{
		SplitCameraNodesRecursive(Child);
	}

	if (Node->Camera.IsValid())
	{
		TSharedPtr<FDatasmithFBXSceneNode> SeparatedChild = MakeShared<FDatasmithFBXSceneNode>();

		SeparatedChild->Name = Node->Name + FString(CAMERA_SUFFIX);
		SeparatedChild->OriginalName = Node->OriginalName;
		SeparatedChild->Camera = Node->Camera;
		SeparatedChild->SplitNodeID = Node->SplitNodeID;

		//Now that the camera is separated from the hierarchy we can apply the roll value without consequences
		SeparatedChild->LocalTransform.SetIdentity();
		SeparatedChild->LocalTransform.ConcatenateRotation(FRotator(-90, -90, -Node->Camera->Roll).Quaternion());

		/** Fix hierarchy

		    P							P
			|							|
			N (camera)		--->		N (node)
		   / \						  / | \
		  C1  C2					 C1 C2 SC (_Camera node)

		P: Parent; N: Node; SC: SeparatedChild; C1,2: Children */
		SeparatedChild->Parent = Node;
		SeparatedChild->Children.Empty();
		Node->Children.Add(SeparatedChild);

		//Clean this Node
		Node->Camera.Reset();
	}
}

void FDatasmithFBXSceneProcessor::RemoveInvisibleNodes()
{
	struct FInvisibleNodesRemover {

		void RemoveNodeTree(TSharedPtr<FDatasmithFBXSceneNode> Node)
		{
			// Recurse to children first. We're iterating in reverse order because iteration may change Children list.
			for (int32 NodeIndex = Node->Children.Num() - 1; NodeIndex >= 0; NodeIndex--)
			{
				RemoveNodeTree(Node->Children[NodeIndex]);
			}
			Node->RemoveNode();
		}

		void RemoveInvisibleNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node)
		{
			// Now check if we can remove this node
			if (!Node->bShouldKeepThisNode && (Node->Visibility < 0.1f))
			{
				RemoveNodeTree(Node);
			}
			else
			{
				// Recurse to children first. We're iterating in reverse order because iteration may change Children list.
				for (int32 NodeIndex = Node->Children.Num() - 1; NodeIndex >= 0; NodeIndex--)
				{
					RemoveInvisibleNodesRecursive(Node->Children[NodeIndex]);
				}
			}
		}
	};

	FInvisibleNodesRemover().RemoveInvisibleNodesRecursive(Scene->RootNode);
}

void FDatasmithFBXSceneProcessor::SimplifyNodeHierarchy()
{
	struct FNodeHierarchySimplifier {

		void RemoveNodeTree(TSharedPtr<FDatasmithFBXSceneNode> Node)
		{
			// Recurse to children first. We're iterating in reverse order because iteration may change Children list.
			for (int32 NodeIndex = Node->Children.Num() - 1; NodeIndex >= 0; NodeIndex--)
			{
				RemoveNodeTree(Node->Children[NodeIndex]);
			}
			Node->RemoveNode();
		}

		void SimplifyHierarchyRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node)
		{
			// Recurse to children first. We're iterating in reverse order because iteration may change Children list.
			for (int32 NodeIndex = Node->Children.Num() - 1; NodeIndex >= 0; NodeIndex--)
			{
				SimplifyHierarchyRecursive(Node->Children[NodeIndex]);
			}

			// Now check if we can remove this node
			if (!Node->bShouldKeepThisNode && !Node->Mesh.IsValid() && Node->LocalTransform.Equals(FTransform::Identity, 0.001f))
			{
				auto Parent = Node->Parent.Pin();
				if (Parent.IsValid())
				{
					// Recurse to children first. We're iterating in reverse order because iteration may change Children list.
					for (int32 NodeIndex = Node->Children.Num() - 1; NodeIndex >= 0; NodeIndex--)
					{
						auto Child = Node->Children[NodeIndex];
						Node->Children.RemoveAt(NodeIndex);
						Parent->Children.Add(Child);
						Child->Parent = Parent;
					}
					Node->RemoveNode();
				}
			}
		}
	};

	FNodeHierarchySimplifier().SimplifyHierarchyRecursive(Scene->RootNode);
}


void FDatasmithFBXSceneProcessor::FixMeshNames()
{
	struct FFixHelper
	{
		void Do(FDatasmithFBXScene* InScene)
		{
			Recurse(InScene->RootNode);
		}

	protected:

		void Recurse(TSharedPtr<FDatasmithFBXSceneNode>& Node)
		{
			if (Node->Mesh.IsValid())
			{
				FString MeshName = Node->Mesh->Name;
				FText Error;
				if (!FFileHelper::IsFilenameValidForSaving(FPaths::GetBaseFilename(MeshName), Error))
				{
					FString MeshNameFixed = MeshName + "_Fixed";
					UE_LOGF(LogDatasmithFBXImport, Warning, "Mesh name \"%ls\" is invalid, renaming to \"%ls\", error: %ls", *MeshName, *MeshNameFixed, *Error.ToString());
					Node->Mesh->Name = MeshNameFixed;
				}
			}

			for (int32 NodeIndex = 0; NodeIndex < Node->Children.Num(); NodeIndex++)
			{
				TSharedPtr<FDatasmithFBXSceneNode>& Child = Node->Children[NodeIndex];

				Recurse(Child);
			}
		}
	};

	FFixHelper Helper;
	Helper.Do(Scene);
}

