// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithFBXTranslatorModule.h"


class FDatasmithFBXTranslatorModule : public IDatasmithFBXTranslatorModule
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FDatasmithFBXTranslatorModule, DatasmithFBXTranslator);
