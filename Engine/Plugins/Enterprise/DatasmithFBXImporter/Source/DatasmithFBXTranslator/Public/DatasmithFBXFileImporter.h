// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "DatasmithFBXScene.h"
#include "FbxImporter.h"

class UDatasmithFBXImportOptions;
struct FDatasmithFBXScene;
struct FDatasmithFBXSceneAnimationData;
struct FDatasmithFBXSceneMaterial;
struct FDatasmithFBXSceneMesh;
struct FDatasmithFBXSceneNode;
struct FDatasmithImportBaseOptions;
struct FFbxUVInfo;

/**
 * Imports an FBX file into the intermediate FBX scene representation
 */
class DATASMITHFBXTRANSLATOR_API FDatasmithFBXFileImporter
{
public:
	FDatasmithFBXFileImporter(FbxScene* InFbxScene, FDatasmithFBXScene* InScene, const UDatasmithFBXImportOptions* InOptions, const FDatasmithImportBaseOptions* InBaseOptions);

	void ImportScene();

protected:
	/** Convert scene coordinate system */
	void ConvertCoordinateSystem();

	/** Create one timeline per animation layer contained in the animation */
	void CreateAnimationTimelines();

	/** Recursively import Fbx scene */
	void TraverseHierarchyNodeRecursively(FbxNode* ParentNode, TSharedPtr<FDatasmithFBXSceneNode>& ParentInfo);

	/** Extract all animation curves from the FBX and pack them into OutScene->AnimNodes.
	Note that only VRED uses this data, as for DeltaGen all animation data comes from .tml aux files */
	void ExtractAnimations(FbxNode* Node);

	/** Allocate FDatasmithFBXSceneMesh */
	TSharedPtr<FDatasmithFBXSceneMesh> ImportMesh(FbxMesh* InMesh, FbxNode* InNode);

	/** Convert Fbx mesh into FRawMesh */
	static void DoImportMesh(FbxMesh* InMesh, FDatasmithFBXSceneMesh* Mesh);

	/** Import an Fbx material */
	TSharedPtr<FDatasmithFBXSceneMaterial> ImportMaterial(FbxSurfaceMaterial* InMaterial);

	static void FindFbxUVChannels(FbxMesh* Mesh, TArray<FFbxUVInfo>& FbxUVs);

	bool IsOddNegativeScale(FbxAMatrix& TotalMatrix);

	void AddCurvesForProperty(FbxProperty InProperty, FbxAnimLayer* InLayer, EDatasmithFBXSceneAnimationCurveType Type, TArray<FDatasmithFBXSceneAnimCurve>& OutCurves);

	void FillCurveFromClipsFile(FbxAnimCurve* InCurve, EDatasmithFBXSceneAnimationCurveType InType, EDatasmithFBXSceneAnimationCurveComponent InComponent);
	FDatasmithFBXSceneAnimCurve CreateNewCurve(FbxAnimCurve* InCurve, EDatasmithFBXSceneAnimationCurveType InType, EDatasmithFBXSceneAnimationCurveComponent InComponent);

	/** Fbx scene which we're importing */
	FbxScene* InScene;

	/** Local scene which we're importing onto */
	FDatasmithFBXScene* OutScene;

	/** All options to configure importer */
	const UDatasmithFBXImportOptions* Options;

	/** Basic options like whether to import geometry or materials at all */
	const FDatasmithImportBaseOptions* BaseOptions;

	/** Map of FbxMesh objects to already imported FDatasmithFBXSceneMesh */
	TMap<FbxMesh*, TSharedPtr<FDatasmithFBXSceneMesh> > ImportedMeshes;

	/** Map of Fbx material objects to imported material structure */
	TMap<FbxSurfaceMaterial*, TSharedPtr<FDatasmithFBXSceneMaterial> > ImportedMaterials;

	/** Stored imported AnimCurves by DSID, which is a specific value inserted into animation curves
	emitted from VRED at time OutScene->TagTime. When we parse a curve from the FBX file, we'll check if
	we can find the corresponding DSID in here, and add the actual animation curve data to it. */
	TMap<int32, TArray<FDatasmithFBXSceneAnimCurve*>> ImportedAnimationCurves;

	bool bDisplayedTwoKeysWarning = false;
};
