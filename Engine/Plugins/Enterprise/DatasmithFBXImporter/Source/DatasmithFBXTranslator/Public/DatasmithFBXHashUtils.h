// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Misc/SecureHash.h"

class FDatasmithFBXHashUtils
{
public:
	static void UpdateHash(FMD5& Hash, int32 Value)
	{
		Hash.Update((const uint8*)&Value, sizeof(Value));
	}

	static void UpdateHash(FMD5& Hash, const FString& String)
	{
		int32 Len = String.Len();
		UpdateHash(Hash, Len);
		if (Len)
		{
			Hash.Update((const uint8*)*String, Len * sizeof(TCHAR));
		}
	}

	static void UpdateHash(FMD5& Hash, const FMD5Hash& Value)
	{
		check(Value.IsValid());
		Hash.Update(Value.GetBytes(), Value.GetSize());
	}

	static void UpdateHash(FMD5& Hash, const FTransform& Value)
	{
		// Convert float values to integers to avoid FP precision errors and difference between +0.0 and -0.0
		FQuat Rotation = Value.GetRotation();
		FVector Translation = Value.GetTranslation();
		FVector Scale = Value.GetScale3D();

		const float Mult = 1000.0f;
		int32 IntValue[10];
		IntValue[0] = Rotation.X * Mult;
		IntValue[1] = Rotation.Y * Mult;
		IntValue[2] = Rotation.Z * Mult;
		IntValue[3] = Rotation.W * Mult;
		IntValue[4] = Translation.X * Mult;
		IntValue[5] = Translation.Y * Mult;
		IntValue[6] = Translation.Z * Mult;
		IntValue[7] = Scale.X * Mult;
		IntValue[8] = Scale.Y * Mult;
		IntValue[9] = Scale.Z * Mult;

		Hash.Update((const uint8*)&IntValue, sizeof(IntValue));
	}

	template<typename T>
	static void UpdateHash(FMD5& Hash, const TArray<T>& Data)
	{
		int32 Count = Data.Num();
		UpdateHash(Hash, Count);
		for (int32 i = 0; i < Count; i++)
		{
			UpdateHash(Hash, Data[i]);
		}
	}

	// Quick hash function for POD-typed array
	template<typename T>
	static void UpdateHashForPODArray(FMD5& Hash, const TArray<T>& Data)
	{
		static_assert(TIsPODType<T>::Value, "This function requires POD array");
		int32 Count = Data.Num();
		UpdateHash(Hash, Count);
		if (Count)
		{
			Hash.Update((const uint8*)Data.GetData(), Data.Num() * sizeof(T));
		}
	}
};

// Make FMD5Hash usable in TMap as a key
inline uint32 GetTypeHash(const FMD5Hash& Hash)
{
	uint32* HashAsInt32 = (uint32*)Hash.GetBytes();
	return HashAsInt32[0] ^ HashAsInt32[1] ^ HashAsInt32[2] ^ HashAsInt32[3];
}

// Allow sorting / comparing hash values
inline bool operator<(const FMD5Hash& A, const FMD5Hash& B)
{
	check(A.IsValid() && B.IsValid());
	return FMemory::Memcmp(A.GetBytes(), B.GetBytes(), A.GetSize()) < 0;
}
