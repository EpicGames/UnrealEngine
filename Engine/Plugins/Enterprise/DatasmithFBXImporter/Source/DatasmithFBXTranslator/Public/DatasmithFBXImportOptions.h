// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithImportOptions.h"

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DatasmithFBXImportOptions.generated.h"

UCLASS(config = EditorPerProjectUserSettings, HideCategories=(DebugProperty))
class DATASMITHFBXTRANSLATOR_API UDatasmithFBXImportOptions : public UDatasmithOptionsBase
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Texture folders", ToolTip="Where to look for textures"))
	TArray<FDirectoryPath> TextureDirs;
};
