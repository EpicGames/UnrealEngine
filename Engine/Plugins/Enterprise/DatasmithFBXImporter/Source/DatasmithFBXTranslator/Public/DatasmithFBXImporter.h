// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IDatasmithActorElement;
class IDatasmithBaseMaterialElement;
class IDatasmithMeshElement;
class IDatasmithScene;
struct FDatasmithFBXScene;
struct FDatasmithFBXSceneMaterial;
struct FDatasmithFBXSceneMesh;
struct FMeshDescription;

#define DATASMITH_FBXIMPORTER_INTERMEDIATE_FORMAT_EXT "intermediate"

using FActorMap = TMap<FName, TArray<TSharedPtr<IDatasmithActorElement>>>;
using FMaterialMap = TMap<FName, TSharedPtr<IDatasmithBaseMaterialElement>>;

/**
* Base class for the VRED and DeltaGen importers, this class is responsible for
* parsing the intermediate FBX scene representation into Datasmith elements
*/
class DATASMITHFBXTRANSLATOR_API FDatasmithFBXImporter
{
public:
	FDatasmithFBXImporter();
	virtual ~FDatasmithFBXImporter();

	void GetGeometriesForMeshElementAndRelease(const TSharedRef<IDatasmithMeshElement> MeshElement, TArray<FMeshDescription>& OutMeshDescriptions);

	void BuildAssetMaps(TSharedRef<IDatasmithScene> Scene, FActorMap& ActorsByOriginalName, FMaterialMap& MaterialsByName);

protected:
	TUniquePtr<FDatasmithFBXScene> IntermediateScene;

	TMap<TSharedPtr<FDatasmithFBXSceneMaterial>, TSharedPtr<IDatasmithBaseMaterialElement>> ImportedMaterials;

	TMap<FName, TSharedPtr<FDatasmithFBXSceneMesh>> MeshNameToFBXMesh;
};
