// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithDefinitions.h" //For light enums

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Curves/RealCurve.h"
#include "Curves/RichCurve.h"
#include "EditorFramework/AssetImportData.h"
#include "Engine/DataTable.h" //For FTableRowBase
#include "MeshDescription.h"
#include "Misc/SecureHash.h"

struct FDatasmithFBXSceneAnimNode;
struct FDatasmithFBXSceneNode;

enum class ELightType : uint8
{
	Point,
	Directional,
	Spot,
	Area
};

enum class EAttenuationType : uint8
{
	None,
	Linear,
	Realistic
};

enum class ENodeType : uint32
{
	Node = 0,
	Switch = 1,
	SharedNode = 2,
	Animated = 4,
	Movable = 8,
	Toggle = 16,
	Material = 32,
};
ENUM_CLASS_FLAGS(ENodeType)

enum class EProjectionType : uint8
{
	Perspective,
	Orghographic
};

enum class ETextureMapType : uint8
{
	UV = 0,
	Planar = 1,
	Triplanar = 2
};

enum class ETextureRepeatMode : uint8
{
	// These match VRED codes
	Repeat = 0,
	Mirror = 9,
	Decal = 18,
	Clamp = 27
};

struct DATASMITHFBXTRANSLATOR_API FDatasmithFBXSceneMaterial
{
	struct FTextureParams
	{
		// Full path of the exported texture
		FString Path;

		// Used for planar projection
		FVector4 Translation;
		FVector4 Rotation;
		FVector4 Scale;

		// Multiplied color
		FVector4 Color;

		// Image space
		// Can't combine these two in a single FVector4
		// because Vector4 parameters are actually float3
		FVector4 Offset;
		FVector4 Repeat;
		float Rotate;  // Degrees

		ETextureMapType ProjectionType;
		ETextureRepeatMode RepeatMode;

		// Exclusively for triplanar projection
		FVector4 TriplanarRotation;
		FVector4 TriplanarOffsetU;
		FVector4 TriplanarOffsetV;
		FVector4 TriplanarRepeatU;
		FVector4 TriplanarRepeatV;
		FVector4 TriplanarTextureSize;
		float TriplanarBlendBias;

		bool bEnabled;

		FTextureParams()
			: Path(FString())
			, Translation(0, 0, 0, 0)
			, Rotation(0, 0, 0, 0)
			, Scale(1, 1, 1, 1)
			, Color(1, 1, 1, 1)
			, Offset(0, 0, 0, 1)
			, Repeat(1, 1, 0, 1)
			, Rotate(0.0)
			, ProjectionType(ETextureMapType::UV)
			, RepeatMode(ETextureRepeatMode::Repeat)
			, TriplanarRotation(0, 0, 0, 1)
			, TriplanarOffsetU(0, 0, 0, 1)
			, TriplanarOffsetV(0, 0, 0, 1)
			, TriplanarRepeatU(1, 1, 1, 1)
			, TriplanarRepeatV(1, 1, 1, 1)
			, TriplanarTextureSize(50.0f, 50.0f, 0.0f, 1.0f)
			, TriplanarBlendBias(0.1f)
			, bEnabled(true)
		{
		}
	};

	FDatasmithFBXSceneMaterial();

	FString Name;
	FString Type;

	// Indexed by param name
	TMap<FString, FVector4> VectorParams;
	TMap<FString, float> ScalarParams;
	TMap<FString, bool> BoolParams;
	TMap<FString, FTextureParams> TextureParams;

	TArray<TWeakPtr<FDatasmithFBXSceneMaterial>> ClonedMaterials;
};

struct DATASMITHFBXTRANSLATOR_API FDatasmithFBXSceneMesh
{
	FDatasmithFBXSceneMesh();
	~FDatasmithFBXSceneMesh();

	FString Name;

	/** Actual mesh geometry */
	FMeshDescription MeshDescription;

	/** Number of materials which could be assigned to this mesh */
	int32 ImportMaterialCount;

	/** Node whose materials will be used if mesh instance will be attached to a node without materials. Used only during Fbx import. */
	TWeakPtr<FDatasmithFBXSceneNode> ImportMaterialsNode;

	const FMD5Hash& GetHash();

	bool HasNormals() const;

	bool HasTangents() const;

	/** Whether the MeshDescription polygon faces have been reversed with respect to
	how they were immediately after import.
	We use this during the merge process, as nodes may share the same mesh and have
	oddnegative and non-oddnegative scales. To compensate for how the vertices are baked
	with the node transforms during merge, we need to flip polygon facing. But since the
	nodes are merged in random order, this helps us coordinate whether we need to reverse
	or not the polygon facing once more. */
	bool bFlippedFaces;

protected:
	/** MD5 of RawMesh contents */
	FMD5Hash Hash;
};

struct FDatasmithFBXSceneLight
{
	FString Name;
	ELightType LightType;
	bool Enabled;
	bool UseTemperature;
	float Temperature;
	float Intensity;
	FLinearColor DiffuseColor;
	FLinearColor UnusedGlossyColor;
	float ConeInnerAngle;
	float ConeOuterAngle;
	bool AreaLightUseConeAngle;
	bool VisualizationVisible;
	EAttenuationType AttenuationType;
	int32 Unit;
	bool UseIESProfile;
	FString IESPath;
	EDatasmithLightShape AreaLightShape;
};

struct DATASMITHFBXTRANSLATOR_API FDatasmithFBXSceneCamera
{
	FDatasmithFBXSceneCamera();
	~FDatasmithFBXSceneCamera();

	double SensorWidth = 36.0f;
	double SensorAspectRatio = 1.7777f;
	double FocalLength = 35.0f;
	double FocusDistance = 100000.0f;

	double NearPlane = 0.1f;
	double FarPlane = 1000.0f;
	double OrthoZoom = 1.0f;
	EProjectionType ProjectionType = EProjectionType::Perspective;

	double Roll = 0.0f;
};

struct DATASMITHFBXTRANSLATOR_API FDatasmithFBXSceneNode : public TSharedFromThis<FDatasmithFBXSceneNode>
{
	FDatasmithFBXSceneNode();
	~FDatasmithFBXSceneNode();

	/** Name of the node. Will be unique. */
	FString Name;

	/** All nodes split from the same original node will have the same SplitNodeID.
		These become actor Tag[1] post-import, so we can match the right node to each variant */
	int32 SplitNodeID;

	/** Node visibility, as set in fbx. */
	float Visibility = 1.0f;
	bool bVisibilityInheritance;

	/** Original name of the node, may be not unique over scene. */
	FString OriginalName;

	/** Transformation of this node relative to its parent. */
	FTransform LocalTransform;

	/** Transform helper objects */
	FVector RotationPivot;
	FVector ScalingPivot;
	FVector RotationOffset;
	FVector ScalingOffset;

	/** Flag indicating that scene optimizer should preserve this node and limit optimization possibilities for it. */
	bool bShouldKeepThisNode;

	// Hierarchy
	TWeakPtr<FDatasmithFBXSceneNode> Parent;
	TArray< TSharedPtr<FDatasmithFBXSceneNode> > Children;

	// Mesh data
	TSharedPtr<FDatasmithFBXSceneMesh> Mesh;
	TArray< TSharedPtr<FDatasmithFBXSceneMaterial> > Materials;

	// Light data
	TSharedPtr<FDatasmithFBXSceneLight> Light;

	// Camera data
	TSharedPtr<FDatasmithFBXSceneCamera> Camera;

	TMap<FString, FString> Metadata;

	FTransform GetTransformRelativeToParent(TSharedPtr<FDatasmithFBXSceneNode>& InParent) const;

	ENodeType GetNodeType() const
	{
		return NodeType;
	}

	/** Get node transform in world coordinate system. */
	FTransform GetWorldTransform() const;

	void AddChild(TSharedPtr<FDatasmithFBXSceneNode>& Child)
	{
		Children.Add(Child);
		Child->Parent = this->AsShared();
	}

	void MoveChildren(TSharedPtr<FDatasmithFBXSceneNode>& NewParent)
	{
		for (auto& Child : Children)
		{
			Child->Parent = NewParent;
		}
		NewParent->Children = MoveTemp(Children);
		Children.Empty();
	}

	/** Remove this node from hierarchy. */
	void RemoveNode();

	/** Get number of children nodes including children of children */
	int32 GetChildrenCountRecursive() const;

	void KeepNode();

	/** This function makes a switch node persistent for scene merging operation */
	void MarkSwitchNode();

	/** This function makes a toggle node persistent for scene merging operation */
	void MarkToggleNode();

	/** Remove special meaning of the node */
	void ResetNodeType();

	const FMD5Hash& GetHash();

	void InvalidateHash()
	{
		Hash = FMD5Hash();
	}

	template<typename F>
	static void Traverse(TSharedPtr<FDatasmithFBXSceneNode> Node, F f)
	{
		if (Node.IsValid())
		{
			f(Node);

			for (auto Child : Node->Children)
			{
				Traverse(Child, f);
			}
		}
	}

protected:
	ENodeType NodeType;

	static int32 NodeCounter;

	/** MD5 of node and its children */
	FMD5Hash Hash;
};

/**
 * Describes all info about the scene that can be extracted from the FBX file
 */
struct DATASMITHFBXTRANSLATOR_API FDatasmithFBXScene
{
	FDatasmithFBXScene();
	~FDatasmithFBXScene();

	using MeshUseCountType = TMap< TSharedPtr<FDatasmithFBXSceneMesh>, int32 >;
	using MaterialUseCountType = TMap< TSharedPtr<FDatasmithFBXSceneMaterial>, int32 >;

	// Flatten all nodes into an array, so that they can be iterated while modifying the
	// hierarchy
	TArray<TSharedPtr<FDatasmithFBXSceneNode>> GetAllNodes();

	struct FStats
	{
		int32 MaterialCount;
		int32 MeshCount;
		int32 GeometryCount;
		int32 NodeCount;

		FStats()
		{
			FMemory::Memzero(*this);
		}
	};
	FStats GetStats();

	TSharedPtr<FDatasmithFBXSceneNode> RootNode;

	TArray<TSharedPtr<FDatasmithFBXSceneMaterial>> Materials;

	TArray<FDatasmithFBXSceneAnimNode> AnimNodes;

	TArray<FName> SwitchObjects;
	TArray<FName> ToggleObjects;
	TArray<FName> ObjectSetObjects;
	TArray<FName> AnimatedObjects;
	TArray<FName> SwitchMaterialObjects;
	TArray<FName> TransformVariantObjects;

	// Time instant where the VRED DSID keys are stored
	float TagTime = FLT_MAX;

	// Native framerate of the animations in fps
	float BaseTime = 24.0f;

	// Playback speed of the animations in fps. In VRED, animations that are natively
	// 1 second long are displayed with total duration 1s * (BaseTime / PlaybackSpeed)
	// The FBX file stores keys with time in seconds, so we have to multiply the key times
	// with (BaseTime / PlaybackSpeed) when passing it to Datasmith to match the final speed
	// in VRED
	float PlaybackSpeed = 24.0f;

	// What we need to multiply translation/scale data with to match the unit
	// scale conversion automatically done when reading the FBX file
	// This is usually 0.1f for DeltaGen and 1.0f for VRED
	double ScaleFactor = 1.0f;

	bool Serialize(FArchive& Ar);
	void RecursiveCollectAllObjects(MeshUseCountType* Meshes, MaterialUseCountType* Materials, int32* NodeCount, const TSharedPtr<FDatasmithFBXSceneNode>& Node) const;
};

enum class EDatasmithFBXSceneAnimationCurveType : uint8
{
	Invalid,
	Translation,
	Rotation,
	Scale,
	Visible
};

enum class EDatasmithFBXSceneAnimationCurveComponent : uint8
{
	X = 0,
	Y = 1,
	Z = 2,
	Num = 3
};

/**
 *	Represents a single key frame of an animation curve. Has interpolation
 *	and tangent information.
 *	Ideally we would use FRichCurveKey, but it's not a BlueprintType, which
 *	we need it to be for serialization
 */
struct FDatasmithFBXSceneAnimPoint
{
	ERichCurveInterpMode InterpolationMode = ERichCurveInterpMode::RCIM_Linear;
	ERichCurveTangentMode TangentMode = ERichCurveTangentMode::RCTM_Auto;
	float Time = 0.0f;
	float Value = 0.0f;
	float ArriveTangent = 0.0f;
	float LeaveTangent = 0.0f;
};

/**
 *	Represents an individual animation curve of a transform property (like
 *	translation X). Has multiple points, representing animation key frames.
 *	Also has an individual DatasmithID (DSID) that can be used to track
 *	the curve through the VRED export process.
 */
struct FDatasmithFBXSceneAnimCurve
{
	// ID of the curve pulled from the FBX. Since VRED doesn't emit any info
	// about the curves in the FBX file, we use this to figure out which block
	// the curve belongs to
	int32 DSID = 0;

	EDatasmithFBXSceneAnimationCurveType Type = EDatasmithFBXSceneAnimationCurveType::Invalid;
	EDatasmithFBXSceneAnimationCurveComponent Component = EDatasmithFBXSceneAnimationCurveComponent::X;
	TArray<FDatasmithFBXSceneAnimPoint> Points;

	// Time of the first true animation key/value pair. Discard everything
	// before this
	float StartTimeSeconds = FLT_MAX;

	bool operator<(const FDatasmithFBXSceneAnimCurve& Other) const
	{
		return DSID < Other.DSID;
	}
};

/**
 *	Represents a set of AnimCurves of an actor. The animation system can
 *	play an AnimBlockUsage of this block, once its part of an AnimClip.
 */
struct FDatasmithFBXSceneAnimBlock
{
	FString Name;
	TArray<FDatasmithFBXSceneAnimCurve> Curves;
};

/**
 *	Describes the set of AnimBlocks that an AnimNode contains
 */
struct FDatasmithFBXSceneAnimNode : public FTableRowBase
{
	FString Name;
	TArray<FDatasmithFBXSceneAnimBlock> Blocks;
};

/**
 *	Used by the VRED importer, this describes how an AnimBlock or AnimClip is used
 *	within an AnimClip. Analogue to an instance of an AnimBlock or AnimClip.
 */
struct FDatasmithFBXSceneAnimUsage
{
	FString AnimName;
	float StartTime = 0.0f;
	float EndTime = 0.0f;
	float IsActive = true;
	bool bIsFlipped = false;
};

/**
 *	Used by the VRED importer, this describes how multiple AnimBlockUsages
 *	are composed to create complex animation sequences involving multiple
 *	actors.
 */
struct FDatasmithFBXSceneAnimClip : public FTableRowBase
{
	FString Name;
	bool bIsFlipped;

	// Blocks and clips that are played when we play this Playable
	TArray<FDatasmithFBXSceneAnimUsage> AnimUsages;
};
