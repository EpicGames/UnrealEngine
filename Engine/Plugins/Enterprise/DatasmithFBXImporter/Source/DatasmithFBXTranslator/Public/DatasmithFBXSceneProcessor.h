// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFBXScene.h"

class DATASMITHFBXTRANSLATOR_API FDatasmithFBXSceneProcessor
{
public:
	FDatasmithFBXSceneProcessor(FDatasmithFBXScene* InScene);

	/** Find duplicated materials and replace them with a single copy */
	void FindDuplicatedMaterials();

	/** Find identical meshes and replace them with a single copy */
	void FindDuplicatedMeshes();

	/** Cleanup unneeded nodes for the whole scene */
	void RemoveLightMapNodes();

	/** Cleanup nodes not meant to be visible */
	void RemoveInvisibleNodes();

	/** Cleanup unneeded nodes for the whole scene */
	void RemoveEmptyNodes()
	{
		RemoveEmptyNodesRecursive(Scene->RootNode);
	}

	/** Cleanup unneeded nodes for hierarchy */
	void RemoveEmptyNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node);

	/** Remove auxilliary geometry generated when exporting from VRED from the whole scene */
	void RemoveTempNodes()
	{
		RemoveTempNodesRecursive(Scene->RootNode);
	}

	/** Remove auxilliary geometry generated when exporting from VRED from a hierarchy */
	void RemoveTempNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node);

	/** Find and mark scene nodes which shouldn't be merged with parent or sibling nodes */
	void FindPersistentNodes();

	/** Collapse node hierarchy . */
	void SimplifyNodeHierarchy();

	/** Fix node names to match output from XmlParser(it converts all whitespace sequences in Content to single space). */
	void FixNodeNames();

	/** Fix invalid mesh names - those that can't be used for asset names(like 'AUX', 'CON' etc)*/
	void FixMeshNames();

	void SplitControlNodes();

	/** Split our lights into a parent that is a simple actor and a child light actor. This because VRED has the
	convention that lights shoot toward -Z, while they shoot toward +X in Unreal. Since we also have Transform
	variants around, we can't just rotate the node and its children once: We need another node */
	void SplitLightNodes()
	{
		SplitLightNodesRecursive(Scene->RootNode);
	}

	/** Recursively splits light actors into a parent simple actor, and a child light actor, with the _Light suffix */
	void SplitLightNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node);

	/** Split camera nodes into a parent that is a simple actor and a child camera actor. This because VRED has the
	convention that cameras shoot toward -Z, while they shoot toward +X in Unreal. Since we also can animate those
	camera nodes (needing a BP_VREDAnimNode), we need to split these camera nodes*/
	void SplitCameraNodes()
	{
		SplitCameraNodesRecursive(Scene->RootNode);
	}

	/** Recursively splits cameras into a parent simple actor, and a child camera actor, with the _Camera suffix */
	void SplitCameraNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node);

protected:
	FDatasmithFBXScene* Scene;
};
