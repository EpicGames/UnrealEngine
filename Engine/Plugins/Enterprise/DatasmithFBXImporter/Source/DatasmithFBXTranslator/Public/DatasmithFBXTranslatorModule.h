// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#define FBXTRANSLATOR_MODULE_NAME TEXT("DatasmithFBXTranslator")

class IDatasmithFBXTranslatorModule : public IModuleInterface
{
public:
	static inline IDatasmithFBXTranslatorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IDatasmithFBXTranslatorModule >(FBXTRANSLATOR_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(FBXTRANSLATOR_MODULE_NAME);
	}
};
