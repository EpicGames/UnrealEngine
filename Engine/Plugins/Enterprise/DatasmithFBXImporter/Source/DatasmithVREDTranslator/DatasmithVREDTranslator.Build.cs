// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class DatasmithVREDTranslator : ModuleRules
	{
		public DatasmithVREDTranslator(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DatasmithCore",
					"EditorFramework",
					"Engine",
					"FBX",
					"LevelSequence",
					"MeshDescription",
					"UnrealEd",
					"XmlParser",
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"DatasmithContent",
					"DatasmithFBXTranslator",
					"DatasmithTranslator",
				}
			);

			//TODO: set this false by default and add option to turn this on for development
			PrivateDefinitions.Add("WITH_VRED_DEBUG_CODE=1");
		}
	}
}
