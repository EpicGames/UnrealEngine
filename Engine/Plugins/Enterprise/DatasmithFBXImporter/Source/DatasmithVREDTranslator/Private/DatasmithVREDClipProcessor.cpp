// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDClipProcessor.h"

#include "DatasmithFBXScene.h"
#include "DatasmithUtils.h"
#include "DatasmithVREDLog.h"

struct FBlockToAdd
{
	FDatasmithFBXSceneAnimBlock Block;
	FDatasmithFBXSceneAnimNode* Parent;
};

void FDatasmithVREDClipProcessor::Process()
{
    BuildMaps();

    // Resolve nested clip delay
    for (auto& Pair : TopLevelClips)
    {
        FString& TopLevelClipName = Pair.Key;
        FDatasmithFBXSceneAnimClip* TopLevelClip = Pair.Value;

        RecursivelyRemoveNestedClipDelay(TopLevelClip, 0.0f);
    }

    // Propagate down flipped flags
    for (auto& Pair : TopLevelClips)
    {
        FString& TopLevelClipName = Pair.Key;
        FDatasmithFBXSceneAnimClip* TopLevelClip = Pair.Value;

        RecursivelyPropagateFlippedToBlockUsages(TopLevelClip, TopLevelClip->bIsFlipped);
    }

	// Add the new blocks we created. Note: This will invalidate most of our maps!
	for (const auto& Pair : OldBlockToReversed)
	{
		const FString& OldBlockName = Pair.Key;
		const TSharedPtr<FBlockToAdd>& BlockToAdd = Pair.Value;

		if (FDatasmithFBXSceneAnimNode* Parent = BlockToAdd->Parent)
		{
			Parent->Blocks.Add(MoveTemp(BlockToAdd->Block));
		}
	}
}

void FDatasmithVREDClipProcessor::BuildMaps()
{
    AllNames.Empty();
    AllClips.Empty();
    TSet<FString> ClipsAsUsages;
    for (FDatasmithFBXSceneAnimClip& Clip : Clips)
    {
        AllClips.Add(Clip.Name, &Clip);

		bool bAlreadyExisted = false;
		AllNames.Add(Clip.Name, &bAlreadyExisted);
		if(bAlreadyExisted)
        {
            UE_LOGF(LogDatasmithVREDImport, Warning, "Found more than one AnimClip/AnimNode named '%ls'. Imported animations may be innacurate!", *Clip.Name);
        }

        for (const FDatasmithFBXSceneAnimUsage& Usage : Clip.AnimUsages)
        {
            ClipsAsUsages.Add(Usage.AnimName);
        }
    }
    TopLevelClips = AllClips;
    for (const FString& UsageName : ClipsAsUsages)
    {
        TopLevelClips.Remove(UsageName);
    }

    AllBlocks.Empty();
	BlockParents.Empty();
    for (FDatasmithFBXSceneAnimNode& AnimNode : AnimNodes)
    {
        for (FDatasmithFBXSceneAnimBlock& Block : AnimNode.Blocks)
        {
            AllBlocks.Add(Block.Name, &Block);
			BlockParents.Add(Block.Name, &AnimNode);

			bool bAlreadyExisted = false;
			AllNames.Add(Block.Name, &bAlreadyExisted);
			if(bAlreadyExisted)
            {
                UE_LOGF(LogDatasmithVREDImport, Warning, "Found more than one AnimClip/AnimNode named '%ls'. Imported animations may be innacurate!", *Block.Name);
            }
        }
    }
}

// VRED has this strange convention where if a top level clip A plays a clip B
// starting at t=2s, clip B's contents will, themselves, all start at t=2s. This
// is not how the sequencer works (which would play clip B starts at t=0),
// so here we remove this "nested clip delay". It is also a lot handier because
// clips can't be reused in VRED, but subsequences can be reused in UnrealEditor
void FDatasmithVREDClipProcessor::RecursivelyRemoveNestedClipDelay(FDatasmithFBXSceneAnimClip* Clip, float Delay)
{
    for (FDatasmithFBXSceneAnimUsage& Usage : Clip->AnimUsages)
    {
        float OldStartTime = Usage.StartTime;

        Usage.StartTime -= Delay;
        Usage.EndTime -= Delay;

        if (FDatasmithFBXSceneAnimClip** Subsequence = AllClips.Find(Usage.AnimName))
        {
            RecursivelyRemoveNestedClipDelay(*Subsequence, OldStartTime);
        }
    }
};

void FDatasmithVREDClipProcessor::RecursivelyPropagateFlippedToBlockUsages(FDatasmithFBXSceneAnimClip* Clip, bool bFlipped)
{
    for (FDatasmithFBXSceneAnimUsage& Usage : Clip->AnimUsages)
    {
        bool bCombinedFlip = Usage.bIsFlipped? !bFlipped : bFlipped;
        if (FDatasmithFBXSceneAnimClip** Subsequence = AllClips.Find(Usage.AnimName))
        {
            RecursivelyPropagateFlippedToBlockUsages(*Subsequence, bCombinedFlip);
        }
        else if (FDatasmithFBXSceneAnimBlock** FoundBlock = AllBlocks.Find(Usage.AnimName))
        {
            // We'll need to have this block usage point at a reversed copy of the block
            if (bCombinedFlip)
            {
				TSharedPtr<FBlockToAdd> ReversedBlock = GetReversedDuplicateAnimBlock(*FoundBlock);
				Usage.AnimName = ReversedBlock->Block.Name;
            }
        }
        else
        {
            UE_LOGF(LogDatasmithVREDImport, Error, "Did not find target of AnimUsage '%ls'!", *Usage.AnimName);
        }
    }
}

TSharedPtr<FBlockToAdd> FDatasmithVREDClipProcessor::GetReversedDuplicateAnimBlock(FDatasmithFBXSceneAnimBlock* Block)
{
	if (Block == nullptr)
	{
		return nullptr;
	}

	if (TSharedPtr<FBlockToAdd>* FoundReversedBlock = OldBlockToReversed.Find(Block->Name))
	{
		return *FoundReversedBlock;
	}

	FDatasmithFBXSceneAnimNode** FoundParent = BlockParents.Find(Block->Name);
	if (!FoundParent || !(*FoundParent))
	{
		return nullptr;
	}

	TSharedPtr<FBlockToAdd> ReversedBlock = MakeShared<FBlockToAdd>();
	ReversedBlock->Parent = *FoundParent;

	// Get a new unique name
	FString Name = FDatasmithUtils::SanitizeObjectName(Block->Name + TEXT("_Reversed"));
	FString Suffix = FString();
	int32 Counter = 0;
	while (AllNames.Contains(Name + Suffix))
	{
		Suffix = TEXT("_") + FString::FromInt(Counter++);
	}
	Name += Suffix;

	ReversedBlock->Block.Name = Name;
	ReversedBlock->Block.Curves = Block->Curves;

	// Find out the block's bounds
	float MinStart = FLT_MAX;
	float MaxEnd = -FLT_MAX;
	for (FDatasmithFBXSceneAnimCurve& Curve : Block->Curves)
	{
		for (FDatasmithFBXSceneAnimPoint& Pt : Curve.Points)
		{
			MinStart = FMath::Min(MinStart, Pt.Time);
			MaxEnd = FMath::Max(MaxEnd, Pt.Time);
		}
	}

	// Actually reverse the points wrt the block bounds
	for (FDatasmithFBXSceneAnimCurve& Curve : ReversedBlock->Block.Curves)
	{
		for (FDatasmithFBXSceneAnimPoint& Pt : Curve.Points)
		{
			Pt.Time = MaxEnd - (Pt.Time - MinStart);
		}

		Curve.Points.Sort([](const FDatasmithFBXSceneAnimPoint& A, const FDatasmithFBXSceneAnimPoint& B)
		{
			return A.Time < B.Time;
		});
	}

	OldBlockToReversed.Add(Block->Name, ReversedBlock);
	return ReversedBlock;
}