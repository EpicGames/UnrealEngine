// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

struct FBlockToAdd;
struct FDatasmithFBXSceneAnimBlock;
struct FDatasmithFBXSceneAnimClip;
struct FDatasmithFBXSceneAnimNode;

// Expects all clips and blocks to have unique names
// Expects DSIDs to have been removed from the curves already
class FDatasmithVREDClipProcessor
{
public:
	FDatasmithVREDClipProcessor(TArray<FDatasmithFBXSceneAnimClip>& InClips, TArray<FDatasmithFBXSceneAnimNode>& InAnimNodes)
		: Clips(InClips)
		, AnimNodes(InAnimNodes)
	{}

    void Process();

private:
    // Prepare some common datastructures to help reference clips and blocks
    void BuildMaps();

	// VRED has this strange convention where if a top level clip A plays a clip B
	// starting at t=2s, clip B's contents will, themselves, all start at t=2s. This
	// is not how the sequencer works (which would play clip B starts at t=0),
	// so here we remove this "nested clip delay". It is also a lot handier because
	// clips can't be reused in VRED, but subsequences can be reused in UnrealEditor
	void RecursivelyRemoveNestedClipDelay (FDatasmithFBXSceneAnimClip* Clip, float Delay);
	void RecursivelyPropagateFlippedToBlockUsages(FDatasmithFBXSceneAnimClip* Clip, bool bFlipped);

	// Creates an AnimBlock that play as if 'Block' had been played back to front, and adds it to the correct AnimNode.
	// Will just return an existing one, if it has been created for this Block already
	TSharedPtr<FBlockToAdd> GetReversedDuplicateAnimBlock(FDatasmithFBXSceneAnimBlock* Block);

private:
	TArray<FDatasmithFBXSceneAnimClip>& Clips;
	TArray<FDatasmithFBXSceneAnimNode>& AnimNodes;

	TSet<FString> AllNames;
	TMap<FString, FDatasmithFBXSceneAnimClip*> AllClips;
	TMap<FString, FDatasmithFBXSceneAnimBlock*> AllBlocks;
	TMap<FString, FDatasmithFBXSceneAnimClip*> TopLevelClips;
	TMap<FString, FDatasmithFBXSceneAnimNode*> BlockParents;
	TMap<FString, TSharedPtr<FBlockToAdd>> OldBlockToReversed;
};
