// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/DataTable.h"

#include "DatasmithVREDImportData.generated.h"

UENUM(BlueprintType)
enum class EVREDCppVariantType : uint8
{
	Unsupported,
	Camera,
	Geometry,
	VariantSet,
	Material,
	Transform,
	Light
};

USTRUCT(BlueprintType)
struct FVREDCppVariantGeometryOption
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> VisibleMeshes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> HiddenMeshes;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantCameraOption
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FVector Location = FVector::ZeroVector;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FRotator Rotation = FRotator::ZeroRotator;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantMaterialOption
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FString Name;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantTransformOption
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FTransform Transform;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantLightOption
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FString Name;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantCamera
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FVREDCppVariantCameraOption> Options;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantGeometry
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> TargetNodes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FVREDCppVariantGeometryOption> Options;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantSet
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> TargetVariantNames;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> ChosenOptions;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FString VariantSetGroupName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> AnimClips;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	bool bSequentialAnimation = false;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantMaterial
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> TargetNodes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FVREDCppVariantMaterialOption> Options;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantTransform
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> TargetNodes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FVREDCppVariantTransformOption> Options;
};

USTRUCT(BlueprintType)
struct FVREDCppVariantLight
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FString> TargetNodes;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	TArray<FVREDCppVariantLightOption> Options;
};

USTRUCT(BlueprintType)
struct FVREDCppVariant : public FTableRowBase
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FString Name;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	EVREDCppVariantType Type = EVREDCppVariantType::Unsupported;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FVREDCppVariantCamera Camera;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FVREDCppVariantGeometry Geometry;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FVREDCppVariantMaterial Material;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FVREDCppVariantSet VariantSet;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FVREDCppVariantTransform Transform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=VRED)
	FVREDCppVariantLight Light;
};

