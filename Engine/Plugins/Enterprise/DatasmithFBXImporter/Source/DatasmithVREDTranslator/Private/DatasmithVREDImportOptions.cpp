// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDImportOptions.h"

#include "Misc/Paths.h"


#define LOCTEXT_NAMESPACE "DatasmithVREDImporter"

UDatasmithVREDImportOptions::UDatasmithVREDImportOptions(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bImportMats(true)
	, bImportVar(true)
	, bCleanVar(true)
	, bImportLightInfo(true)
	, bImportClipInfo(true)
{
}

namespace VREDImportOptionsImpl
{
	FString FindBestFile(const FString& FBXFileWithoutExt, const FString& Extension)
	{
		const FString FBXDirectory = FPaths::GetPath(FBXFileWithoutExt);

		FString VarPathStr = FPaths::SetExtension(FBXFileWithoutExt, Extension);
		if (FPaths::FileExists(VarPathStr))
		{
			return VarPathStr;
		}

		return FString();
	}
}

void UDatasmithVREDImportOptions::ResetPaths(const FString& InFBXFilename, bool bJustEmptyPaths)
{
	// Handle file.fbx and file.fbx.intermediate
	FString PathNoExt = FPaths::ChangeExtension(FPaths::ChangeExtension(InFBXFilename, ""), "");

	if (MatsPath.FilePath.IsEmpty() || !bJustEmptyPaths)
	{
		MatsPath.FilePath = VREDImportOptionsImpl::FindBestFile(PathNoExt, TEXT("mats"));
	}
	if (VarPath.FilePath.IsEmpty() || !bJustEmptyPaths)
	{
		VarPath.FilePath = VREDImportOptionsImpl::FindBestFile(PathNoExt, TEXT("var"));
	}
	if (LightInfoPath.FilePath.IsEmpty() || !bJustEmptyPaths)
	{
		LightInfoPath.FilePath = VREDImportOptionsImpl::FindBestFile(PathNoExt, TEXT("lights"));
	}
	if (ClipInfoPath.FilePath.IsEmpty() || !bJustEmptyPaths)
	{
		ClipInfoPath.FilePath = VREDImportOptionsImpl::FindBestFile(PathNoExt, TEXT("clips"));
	}

	if (TextureDirs.Num() == 0 || !bJustEmptyPaths)
	{
		FString TexturesDirStr = FPaths::GetPath(PathNoExt) / TEXT("Textures");
		if (FPaths::DirectoryExists(TexturesDirStr))
		{
			TextureDirs.SetNum(1);
			TextureDirs[0].Path = TexturesDirStr;
		}
	}
}

#undef LOCTEXT_NAMESPACE
