// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFBXImportOptions.h"


#include "DatasmithVREDImportOptions.generated.h"

class UDatasmithFBXSceneImportData;

UCLASS(config=EditorPerProjectUserSettings, HideCategories=(Debug))
class UDatasmithVREDImportOptions : public UDatasmithFBXImportOptions
{
	GENERATED_UCLASS_BODY()

public:
	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category= AssetImporting, meta=(DisplayName="Import materials file", ToolTip="Uses the *.mats file saved alongside the exported FBX for a more accurate material reproduction"))
	bool bImportMats;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Materials file path", EditCondition = "bImportMats", ToolTip="Path to the *.mats file. By default it will search for a *.mats file in the same folder as the FBX file, with the same base filename as it", FilePathFilter="mats"))
	FFilePath MatsPath;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category= AssetImporting, meta=(DisplayName="Import variants file", ToolTip="Uses the *.var file saved alongside the exported FBX"))
	bool bImportVar;

	UPROPERTY(config, EditAnywhere, AdvancedDisplay, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Clean variants", ToolTip="Removes empty variants, variant sets and invalid options. All discarded items will be logged to console."))
	bool bCleanVar;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Variants file path", EditCondition = "bImportVar", ToolTip="Path to the *.var file. By default it will search for a *.var file in the same folder as the FBX file, with the same base filename as it", FilePathFilter="var"))
	FFilePath VarPath;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Import lights file", ToolTip="Uses the *.lights file saved alongside the exported FBX to import extra information about lights not saved in the FBX file"))
	bool bImportLightInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Lights file path", EditCondition = "bImportLightInfo", ToolTip="Path to the *.lights file. By default it will search for a *.light file in the same folder as the FBX file, with the same base filename as it", FilePathFilter="lights"))
	FFilePath LightInfoPath;

	UPROPERTY(config, EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="Import clips file", ToolTip="Uses the *.clips file saved alongside the exported FBX to import information about animation clips and blocks, mirroring VRED's animation system"))
	bool bImportClipInfo;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=AssetImporting, meta=(DisplayName="AnimClips file path", EditCondition = "bImportClipInfo", ToolTip="Path to the *.clips file. By default it will search for a *.clips file in the same folder as the FBX file, with the same base filename as it", FilePathFilter="clips"))
	FFilePath ClipInfoPath;

public:
	void ResetPaths(const FString& InFBXFilename, bool bJustEmptyPaths=true);
};
