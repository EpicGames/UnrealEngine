// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDImporter.h"

#include "Async/AsyncWork.h"
#include "DatasmithAnimationElements.h"
#include "DatasmithFBXFileImporter.h"
#include "DatasmithTypes.h"
#include "DatasmithVREDClipProcessor.h"
#include "DatasmithVREDImportData.h"
#include "DatasmithVREDImporterAuxFiles.h"
#include "DatasmithVREDImportOptions.h"
#include "DatasmithVREDLog.h"
#include "DatasmithVREDSceneProcessor.h"
#include "DatasmithVREDVariantConverter.h"

#include "DatasmithSceneFactory.h"
#include "DatasmithUtils.h"
#include "DatasmithVariantElements.h"

#include "FbxImporter.h"
#include "HAL/FileManager.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include <core/fbxproperty.h>
#include <scene/fbxdocumentinfo.h>
#include <scene/fbxscene.h>
#include <utils/fbxrenamingstrategyutilities.h>

DEFINE_LOG_CATEGORY(LogDatasmithVREDImport);

#define LOCTEXT_NAMESPACE "DatasmithVREDImporter"

// Use some suffix to make names unique
#define UNIQUE_NAME_SUFFIX TEXT(NAMECLASH1_KEY)

// Do not allow mesh names longer than this value
#define MAX_MESH_NAME_LENGTH 48

// Internally, attachment performed in USceneComponent::AttachToComponent(). This function determines LastAttachIndex
// using some logic, then inserts new actor as FIRST element in child array, i.e. adding actors 1,2,3 will result
// these actors in reverse order (3,2,1). We're using logic which prevents this by iterating children in reverse order.
#define REVERSE_ATTACH_ORDER 1

#define MATCHING_EXPORTER_VERSION TEXT("4")

struct FImportedAnim
{
	FString Name;
	TSharedPtr<IDatasmithLevelSequenceElement> ImportedSequence;
	float OriginalStartSeconds;
	float OriginalEndSeconds;
};

// If an AnimBlock was ever split in scene processing (e.g. due to a node having a rotation pivot)
// a CombinedAnimBlock will contain the names of the AnimNodes originating from the split, as well
// as pointers to all blocks that were created for the split. These will all have the same name
struct FCombinedAnimBlock
{
	TMap<FString, FDatasmithFBXSceneAnimBlock*> NodeNameToBlock;

	// Returns the combined, min start time for all blocks we have
	float GetStartSeconds()
	{
		float MinStartTime = FLT_MAX;

		for (const auto& Pair : NodeNameToBlock)
		{
			const FString& AnimNodeName = Pair.Key;
			const FDatasmithFBXSceneAnimBlock* Block = Pair.Value;

			for (const FDatasmithFBXSceneAnimCurve& Curve : Block->Curves)
			{
				if (!FMath::IsNearlyEqual(Curve.StartTimeSeconds, FLT_MAX))
				{
					MinStartTime = FMath::Min(MinStartTime, Curve.StartTimeSeconds);
				}
				else
				{
					int32 NumPts = Curve.Points.Num();
					if (NumPts > 0)
					{
						MinStartTime = FMath::Min(MinStartTime, Curve.Points[0].Time);
					}
				}
			}
		}

		return MinStartTime;
	}

	// Returns the combined, max end time for all blocks we have
	float GetEndSeconds()
	{
		float MaxEndTime = -FLT_MAX;

		for (const auto& Pair : NodeNameToBlock)
		{
			const FString& AnimNodeName = Pair.Key;
			const FDatasmithFBXSceneAnimBlock* Block = Pair.Value;

			for (const FDatasmithFBXSceneAnimCurve& Curve : Block->Curves)
			{
				int32 NumPts = Curve.Points.Num();
				if (NumPts > 0)
				{
					MaxEndTime = FMath::Max(MaxEndTime, Curve.Points[NumPts-1].Time);
				}
			}
		}

		return MaxEndTime;
	}
};

class FAsyncReleaseFbxScene : public FNonAbandonableTask
{
public:
	FAsyncReleaseFbxScene( UnFbx::FFbxImporter* InFbxImporter )
		: FbxImporter( InFbxImporter )
	{
	}

	void DoWork()
	{
		FbxImporter->ReleaseScene();
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FAsyncReleaseFbxScene, STATGROUP_ThreadPoolAsyncTasks);
	}

private:
	UnFbx::FFbxImporter* FbxImporter;
};

FDatasmithVREDImporter::FDatasmithVREDImporter(TSharedRef<IDatasmithScene>& OutScene, UDatasmithVREDImportOptions* InOptions)
	: FDatasmithFBXImporter()
	, DatasmithScene(OutScene)
	, ImportOptions(InOptions)
{
}

FDatasmithVREDImporter::~FDatasmithVREDImporter()
{
}

void FDatasmithVREDImporter::SetImportOptions(UDatasmithVREDImportOptions* InOptions)
{
	ImportOptions = InOptions;
}

bool FDatasmithVREDImporter::OpenFile(const FString& FilePath)
{
	// Need to do this before we parse the FBX file as we'll need the stuff in the .clips file to properly
	// identify the FBX animation curves
	ParseAuxFiles(FilePath);

	if (!ParseFbxFile(FilePath))
	{
		return false;
	}

	FDatasmithFBXScene::FStats Stats = IntermediateScene->GetStats();
	UE_LOGF(LogDatasmithVREDImport, Log, "Scene %ls has %d nodes, %d geometries, %d meshes, %d materials",
		*FilePath, Stats.NodeCount, Stats.GeometryCount, Stats.MeshCount, Stats.MaterialCount);

	ProcessScene();

	Stats = IntermediateScene->GetStats();
	UE_LOGF(LogDatasmithVREDImport, Log, "Processed scene %ls has %d nodes, %d geometries, %d meshes, %d materials",
		*FilePath, Stats.NodeCount, Stats.GeometryCount, Stats.MeshCount, Stats.MaterialCount);

	return true;
}

bool inline CheckFbxScene(fbxsdk::FbxScene* Scene)
{
	/* In the DatasmithExporter VRED plugin we place these extra empty nodes to signal
	metadata, since the API doesn't provide any better way of doing so.

	VRED2Unreal is an illegal node name in the exporter, so if we ever find a node with
	this name, we can be sure it's made by the exporter itself to signal metadata.

	The metadata hierarchy is as follows:

	Root node
		|
	(Node named "VRED2Unreal")
		|
	(Node whose name is the DatasmithExporter version string, e.g. 1_4_2)
		|
	(Node whose name is VRED's version string or "Unknown", e.g. 11_0)
	*/

	fbxsdk::FbxNode* VRED2UnrealNode = nullptr;
	fbxsdk::FbxNode* ExporterVersionNode = nullptr;
	fbxsdk::FbxNode* VREDVersionNode = nullptr;

	VRED2UnrealNode = Scene->FindNodeByName("VRED2Unreal");
	if (VRED2UnrealNode != nullptr)
	{
		int32 NodeCount = VRED2UnrealNode->GetChildCount();
		if (NodeCount == 1)
		{
			ExporterVersionNode = VRED2UnrealNode->GetChild(0);
			if (ExporterVersionNode != nullptr)
			{
				NodeCount = ExporterVersionNode->GetChildCount();
				if (NodeCount == 1)
				{
					VREDVersionNode = ExporterVersionNode->GetChild(0);
				}
			}
		}
	}

	FString VREDVersion;
	FString ExporterVersion;

	if (VREDVersionNode != nullptr)
	{
		VREDVersion = UTF8_TO_TCHAR(VREDVersionNode->GetName());
		Scene->RemoveNode(VREDVersionNode);
	}

	if (ExporterVersionNode != nullptr)
	{
		ExporterVersion = UTF8_TO_TCHAR(ExporterVersionNode->GetName());
		Scene->RemoveNode(ExporterVersionNode);
	}

	if (VRED2UnrealNode != nullptr)
	{
		Scene->RemoveNode(VRED2UnrealNode);
	}

	if (ExporterVersion.IsEmpty() || VREDVersion.IsEmpty())
	{
		return false;
	}

	UE_LOGF(LogDatasmithVREDImport, Log, "Scene was exported from VRED version '%ls', with DatasmithExporter plugin version '%ls'", *VREDVersion, *ExporterVersion);

	if(ExporterVersion != MATCHING_EXPORTER_VERSION)
	{
		UE_LOGF(LogDatasmithVREDImport, Warning, "Imported data may be innacurate! For the best results, please re-export the scene from VRED with version '%ls' of the DatasmithExporter plugin!", MATCHING_EXPORTER_VERSION);
	}

	return true;
}

// Uses the names of imported nodes to clean up any empty variants, variant sets or variants that target non-existent
// nodes or materials
void CleanImportedVariants(TArray<FVREDCppVariant>& VariantSwitches, const FDatasmithFBXScene* VREDScene)
{
	// Catalogue existing items
	TSet<FString> NodeNames;
	FDatasmithFBXSceneNode::Traverse(VREDScene->RootNode, [&](const TSharedPtr<FDatasmithFBXSceneNode>& Node)
	{
		NodeNames.Add(Node->OriginalName); // OriginalName will become our NameID
	});
	TSet<FString> MaterialNames;
	for (const TSharedPtr<FDatasmithFBXSceneMaterial>& Mat : VREDScene->Materials)
	{
		MaterialNames.Add(Mat->Name);
	}

	UE_LOGF(LogDatasmithVREDImport, Log, "Attempting to remove invalid or empty variants, variant sets and variant set references. This step can be disabled on the import options menu.");

	// Cleanup variants
	for (int32 VarIndex = VariantSwitches.Num() - 1; VarIndex >= 0; VarIndex--)
	{
		FVREDCppVariant& Var = VariantSwitches[VarIndex];
		FString& VarName = Var.Name;

		switch (Var.Type)
		{
		case EVREDCppVariantType::Camera:
		{
			UE_LOGF(LogDatasmithVREDImport, Log, "Removed camera variant %ls", *Var.Name);
			VariantSwitches.RemoveAt(VarIndex);
			break;
		}
		case EVREDCppVariantType::Geometry:
		{
			FVREDCppVariantGeometry& Geom = Var.Geometry;

			// Clean options
			for (int32 OptionIndex = Geom.Options.Num() - 1; OptionIndex >= 0; OptionIndex--)
			{
				FVREDCppVariantGeometryOption& Option = Geom.Options[OptionIndex];

				Option.HiddenMeshes.RemoveAll([&NodeNames](FString& HiddenMeshName)
				{
					return !NodeNames.Contains(HiddenMeshName);
				});
				Option.VisibleMeshes.RemoveAll([&NodeNames](FString& HiddenMeshName)
				{
					return !NodeNames.Contains(HiddenMeshName);
				});

				if (Option.HiddenMeshes.Num() + Option.VisibleMeshes.Num() == 0)
				{
					UE_LOGF(LogDatasmithVREDImport, Log, "Removed option %ls from geometry variant %ls", *Option.Name, *VarName);
					Geom.Options.RemoveAt(OptionIndex);
				}
			}

			// Clean target nodes
			Geom.TargetNodes.RemoveAll([&NodeNames](FString& TargetNodeName)
			{
				return !NodeNames.Contains(TargetNodeName);
			});

			// Clean ourselves
			if (Geom.TargetNodes.Num() + Geom.Options.Num() == 0)
			{
				UE_LOGF(LogDatasmithVREDImport, Log, "Removed geometry variant %ls", *VarName);
				VariantSwitches.RemoveAt(VarIndex);
			}
			break;
		}
		case EVREDCppVariantType::Light:
		{
			FVREDCppVariantLight& Light = Var.Light;
			Light.TargetNodes.RemoveAll([&NodeNames](FString& TargetNodeName)
			{
				return !NodeNames.Contains(TargetNodeName);
			});

			if (Light.TargetNodes.Num() == 0)
			{
				UE_LOGF(LogDatasmithVREDImport, Log, "Removed light variant %ls", *VarName);
				VariantSwitches.RemoveAt(VarIndex);
			}
			break;
		}
		case EVREDCppVariantType::Material:
		{
			FVREDCppVariantMaterial& Mat = Var.Material;
			Mat.Options.RemoveAll([&MaterialNames, &VarName](FVREDCppVariantMaterialOption& Option)
			{
				if (!MaterialNames.Contains(Option.Name))
				{
					UE_LOGF(LogDatasmithVREDImport, Log, "Removed option %ls from material variant %ls", *Option.Name, *VarName);
					return true;
				}
				return false;
			});

			Mat.TargetNodes.RemoveAll([&NodeNames](FString& TargetNodeName)
			{
				return !NodeNames.Contains(TargetNodeName);
			});

			if (Mat.Options.Num() == 0 || Mat.TargetNodes.Num() == 0)
			{
				UE_LOGF(LogDatasmithVREDImport, Log, "Removed material variant %ls", *VarName);
				VariantSwitches.RemoveAt(VarIndex);
			}
			break;
		}
		case EVREDCppVariantType::Transform:
		{
			FVREDCppVariantTransform& Trans = Var.Transform;
			Trans.TargetNodes.RemoveAll([&NodeNames](FString& TargetNodeName)
			{
				return !NodeNames.Contains(TargetNodeName);
			});

			if (Trans.Options.Num() == 0 || Trans.TargetNodes.Num() == 0)
			{
				UE_LOGF(LogDatasmithVREDImport, Log, "Removed transform variant %ls", *VarName);
				VariantSwitches.RemoveAt(VarIndex);
			}
			break;
		}
		case EVREDCppVariantType::VariantSet:
			break;
		case EVREDCppVariantType::Unsupported:  // Intended fallthrough
		default:
			VariantSwitches.RemoveAt(VarIndex);
			break;
		}
	}

	// Now that we may have removed some variants, we need to update invalid variant sets
	for (int32 VarIndex = VariantSwitches.Num() - 1; VarIndex >= 0; VarIndex--)
	{
		FVREDCppVariant& Var = VariantSwitches[VarIndex];
		FString& VarName = Var.Name;
		if (Var.Type != EVREDCppVariantType::VariantSet)
		{
			continue;
		}

		FVREDCppVariantSet& VarSet = Var.VariantSet;

		for (int32 OptionIndex = VarSet.TargetVariantNames.Num() - 1; OptionIndex >= 0; OptionIndex--)
		{
			FString& TargetVariantName = VarSet.TargetVariantNames[OptionIndex];
			FString& ChosenOptionName = VarSet.ChosenOptions[OptionIndex];
			bool bValidOption = true;

			FVREDCppVariant* TargetVarPtr = VariantSwitches.FindByPredicate([&TargetVariantName](const FVREDCppVariant& Var)
			{
				return Var.Name == TargetVariantName;
			});

			// Check if target variant exist
			if (TargetVarPtr)
			{
				FVREDCppVariant TargetVar = *TargetVarPtr;

				// Check if option exists within target variant
				switch (TargetVar.Type)
				{
				case EVREDCppVariantType::Camera:
					bValidOption = TargetVar.Camera.Options.ContainsByPredicate([&ChosenOptionName](const FVREDCppVariantCameraOption& Option)
					{
						return Option.Name == ChosenOptionName;
					});
					break;
				case EVREDCppVariantType::Geometry:
					bValidOption = TargetVar.Geometry.Options.ContainsByPredicate([&ChosenOptionName](const FVREDCppVariantGeometryOption& Option)
					{
						return Option.Name == ChosenOptionName;
					});
					break;
				case EVREDCppVariantType::Light:
					bValidOption = (ChosenOptionName == TEXT("!Enable") || ChosenOptionName == TEXT("!Disable"));
					break;
				case EVREDCppVariantType::Material:
					bValidOption = TargetVar.Material.Options.ContainsByPredicate([&ChosenOptionName](const FVREDCppVariantMaterialOption& Option)
					{
						return Option.Name == ChosenOptionName;
					});
					break;
				case EVREDCppVariantType::Transform:
					bValidOption = TargetVar.Transform.Options.ContainsByPredicate([&ChosenOptionName](const FVREDCppVariantTransformOption& Option)
					{
						return Option.Name == ChosenOptionName;
					});
					break;
				case EVREDCppVariantType::Unsupported:
					bValidOption = false;
					break;
				case EVREDCppVariantType::VariantSet:
					bValidOption = false;
					break;
				default:
					bValidOption = false;
					break;
				}
			}
			else
			{
				bValidOption = false;
			}

			if (!bValidOption)
			{
				UE_LOGF(LogDatasmithVREDImport, Log, "Removed reference to variant %ls with chosen option %ls from variant set %ls", *TargetVariantName, *ChosenOptionName, *Var.Name);
				VarSet.TargetVariantNames.RemoveAt(OptionIndex);
				VarSet.ChosenOptions.RemoveAt(OptionIndex);
			}
		}

		if (VarSet.ChosenOptions.Num() == 0 || VarSet.TargetVariantNames.Num() == 0)
		{
			VariantSwitches.RemoveAt(VarIndex);
		}
	}
}

bool FDatasmithVREDImporter::ParseFbxFile(const FString& FBXPath)
{
	UnFbx::FFbxImporter* FbxImporter = UnFbx::FFbxImporter::GetInstance();
	UnFbx::FBXImportOptions* GlobalImportSettings = FbxImporter->GetImportOptions();
	UnFbx::FBXImportOptions::ResetOptions(GlobalImportSettings);

	if (!FbxImporter->ImportFromFile(FBXPath, FPaths::GetExtension(FBXPath), false))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Error parsing FBX file: %ls", FbxImporter->GetErrorMessage());

		( new FAutoDeleteAsyncTask< FAsyncReleaseFbxScene >( FbxImporter ) )->StartBackgroundTask();
		return false;
	}

	// Check if the FBX was imported through the exporter plugin
	bool bIsVREDScene = CheckFbxScene(FbxImporter->Scene);
	if (!bIsVREDScene)
	{
		EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(LOCTEXT("DatasmithVREDImporter", "Warning:\n\nThe FBX scene '{0}' has not been exported through the DatasmithExporter VRED plugin. Continuing the import process will likely lead to errors. \n\nWould you like to continue importing?"), FText::FromString(FPaths::ConvertRelativePathToFull(FBXPath))));

		if (Result == EAppReturnType::No)
		{
			( new FAutoDeleteAsyncTask< FAsyncReleaseFbxScene >( FbxImporter ) )->StartBackgroundTask();
			return false;
		}
	}

	// TODO: BaseOptions
	FDatasmithImportBaseOptions DefaultBaseOptions;

	FDatasmithFBXFileImporter Importer(FbxImporter->Scene, IntermediateScene.Get(), ImportOptions, &DefaultBaseOptions);
	Importer.ImportScene();

	// TODO: BaseOptions
	//if (bIsVREDScene && Context.Options->BaseOptions.bIncludeAnimation && !Options->bImportClipInfo && VREDScene->AnimNodes.Num() > 0)
	//{
	//	Context.LogWarning(FText::FromString(TEXT("Importing a VRED scene with animations but without a .clips file!. During the VRED export process the animation curves are modified, and intended to be parsed with the information obtained from the .clips file. Without that data the parsed animation curves might contain artifacts and strange behaviour. It is recommended to provide a .clips file or export a regular FBX scene from VRED.")));
	//}

	if ( FbxImporter->Scene && FbxImporter->Scene->GetSceneInfo() )
	{
		DatasmithScene->SetProductName( UTF8_TO_TCHAR( FbxImporter->Scene->GetSceneInfo()->Original_ApplicationName.Get().Buffer() ) );
		DatasmithScene->SetProductVersion( UTF8_TO_TCHAR( FbxImporter->Scene->GetSceneInfo()->Original_ApplicationVersion.Get().Buffer() ) );
		DatasmithScene->SetVendor( UTF8_TO_TCHAR( FbxImporter->Scene->GetSceneInfo()->Original_ApplicationVendor.Get().Buffer() ) );
	}

	( new FAutoDeleteAsyncTask< FAsyncReleaseFbxScene >( FbxImporter ) )->StartBackgroundTask();
	return true;
}

void FDatasmithVREDImporter::UnloadScene()
{
}

// This will replace the AnimCurves in Dst with the AnimCurves in Src that have the same DSID.
// The curve points, type and component will be moved across
void MoveCurvesByDSID(TArray<FDatasmithFBXSceneAnimNode>& Dst, TArray<FDatasmithFBXSceneAnimNode>& Src)
{
	TMap<int32, FDatasmithFBXSceneAnimCurve*> CurveData;
	for (FDatasmithFBXSceneAnimNode& AnimNode : Src)
	{
		for (FDatasmithFBXSceneAnimBlock& Block : AnimNode.Blocks)
		{
			for (FDatasmithFBXSceneAnimCurve& Curve : Block.Curves)
			{
				if (CurveData.Contains(Curve.DSID))
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "Warning: More than one curve with DSID %d", Curve.DSID);
				}

				CurveData.Add(Curve.DSID, &Curve);
			}
		}
	}

	for (FDatasmithFBXSceneAnimNode& AnimNode : Dst)
	{
		for (FDatasmithFBXSceneAnimBlock& Block : AnimNode.Blocks)
		{
			for (FDatasmithFBXSceneAnimCurve& Curve : Block.Curves)
			{
				if (FDatasmithFBXSceneAnimCurve** FoundCurve = CurveData.Find(Curve.DSID))
				{
					Curve.Type = (*FoundCurve)->Type;
					Curve.Component = (*FoundCurve)->Component;
					Curve.Points = MoveTemp((*FoundCurve)->Points);
				}
				else
				{
					// This is OK, as a bunch of curves won't make it to the FBX file for being unused/empty
					//UE_LOGF(LogDatasmithVREDImport, Warning, "Warning: Clips file had no description of curve with DSID %d", Curve.DSID);
				}
			}
		}
	}
}

// When we retrieve the animation keys from the FBX file, they will be correctly in seconds (not frames), and will end up
// baked to 30fps when passing to Datasmith.
// In VRED though, while they are also stored in seconds internally, the user controls two numbers: BaseTime, which
// is the native framerate of the animation; and PlaybackSpeed, which is the rate with which the animations are played.
// The ratio BaseTime/PlaybackSpeed is the DurationMultiplier we pass in here. This ensures that a 1 second VRED animation
// at 96 base fps (e.g. with keys at frame 0 and 96) played back at 24 fps takes 4 seconds total, as it should.
// Note how we don't care about framerate and playback speed here specifically, only the ratio: A 1 second animation
// with base framerate of 24 fps played back at 24 fps will take 1 second, the same as if it had a base of 96 fps and
// were played back at 96 fps
void ApplyCurveDurationMultiplier(TArray<FDatasmithFBXSceneAnimNode>& AnimNodes, TArray<FDatasmithFBXSceneAnimClip>& AnimClips, double DurationMultiplier)
{
	for (FDatasmithFBXSceneAnimNode& AnimNode : AnimNodes)
	{
		for (FDatasmithFBXSceneAnimBlock& Block : AnimNode.Blocks)
		{
			for (FDatasmithFBXSceneAnimCurve& Curve : Block.Curves)
			{
				for (FDatasmithFBXSceneAnimPoint& Pt : Curve.Points)
				{
					Pt.Time *= DurationMultiplier;
				}
			}
		}
	}

	for (FDatasmithFBXSceneAnimClip& Clip : AnimClips)
	{
		for (FDatasmithFBXSceneAnimUsage& Usage : Clip.AnimUsages)
		{
			Usage.StartTime *= DurationMultiplier;
			Usage.EndTime *= DurationMultiplier;
		}
	}
}

void FDatasmithVREDImporter::ParseAuxFiles(const FString& FBXPath)
{
	// TODO: BaseOptions
	if(/*Context.Options->BaseOptions.bIncludeLight && */ImportOptions->bImportLightInfo)
	{
		FDatasmithVREDImportLightsResult LightsResult = FDatasmithVREDAuxFiles::ParseLightsFile(ImportOptions->LightInfoPath.FilePath);
		ParsedLightsInfo = LightsResult.Lights;
	}

	// TODO: BaseOptions
	if (/*Context.Options->BaseOptions.bIncludeAnimation && */ImportOptions->bImportClipInfo)
	{
		FDatasmithVREDImportClipsResult ClipsResult = FDatasmithVREDAuxFiles::ParseClipsFile(ImportOptions->ClipInfoPath.FilePath);
		IntermediateScene->AnimNodes = MoveTemp(ClipsResult.AnimNodes);
		IntermediateScene->TagTime = ClipsResult.KeyTime;
		IntermediateScene->BaseTime = ClipsResult.BaseTime;
		IntermediateScene->PlaybackSpeed = ClipsResult.PlaybackSpeed;
		ParsedAnimClips = MoveTemp(ClipsResult.AnimClips);
	}

	// Import variants data
	if (ImportOptions->bImportVar)
	{
		FDatasmithVREDImportVariantsResult VarResult = FDatasmithVREDAuxFiles::ParseVarFile(ImportOptions->VarPath.FilePath);
		for (FString Name : VarResult.SwitchObjects)
		{
			IntermediateScene->SwitchObjects.Add(FName(*Name));
		}
		for (FString Name : VarResult.SwitchMaterialObjects)
		{
			IntermediateScene->SwitchMaterialObjects.Add(FName(*Name));
		}
		for (FString Name : VarResult.TransformVariantObjects)
		{
			IntermediateScene->TransformVariantObjects.Add(FName(*Name));
		}
		ParsedVariants = VarResult.VariantSwitches;
	}

	// TODO: BaseOptions
	// Import materials from .mats file
	if (ImportOptions->bImportMats)
	{
		FDatasmithVREDImportMatsResult MatsResult = FDatasmithVREDAuxFiles::ParseMatsFile(ImportOptions->MatsPath.FilePath);
		ParsedMats = MatsResult.Mats;
	}
}

void FDatasmithVREDImporter::ProcessScene()
{
	// TODO: BaseOptions
	if (/*Context.Options->BaseOptions.bIncludeAnimation && */ImportOptions->bImportClipInfo)
	{
		ApplyCurveDurationMultiplier(IntermediateScene->AnimNodes, ParsedAnimClips, IntermediateScene->BaseTime / IntermediateScene->PlaybackSpeed);

		FDatasmithVREDClipProcessor ClipProcessor(ParsedAnimClips, IntermediateScene->AnimNodes);
		ClipProcessor.Process();
	}

	// TODO: BaseOptions
	// Keep track of the names of animated objects
	// This is done here as these might come from the FBX file or .clips file
	// if (Context.Options->BaseOptions.bIncludeAnimation)
	{
		for (const FDatasmithFBXSceneAnimNode& AnimNode : IntermediateScene->AnimNodes)
		{
			IntermediateScene->AnimatedObjects.Add(FName(*AnimNode.Name));
		}
	}

	FDatasmithVREDSceneProcessor Processor(IntermediateScene.Get());

	// TODO: BaseOptions
	// if (Context.Options->BaseOptions.bIncludeLight)
	{
		Processor.AddExtraLightInfo(&ParsedLightsInfo);
	}

	// TODO: BaseOptions
	if (/*Context.Options->BaseOptions.bIncludeMaterial && */ImportOptions->bImportMats)
	{
		Processor.AddMatsMaterials(&ParsedMats);
	}

	Processor.RemoveLightMapNodes();

	Processor.FindPersistentNodes();

	// TODO: BaseOptions
	// if (Context.Options->BaseOptions.bIncludeLight)
	{
		Processor.SplitLightNodes();
	}

	// TODO: BaseOptions
	// if (Context.Options->BaseOptions.bIncludeCamera)
	{
		Processor.SplitCameraNodes();
	}

	Processor.DecomposeRotationPivots();

	Processor.DecomposeScalingPivots();

	Processor.FindDuplicatedMaterials();

	Processor.FindDuplicatedMeshes();

	Processor.RemoveEmptyNodes();

	Processor.RemoveTempNodes();

	Processor.FixMeshNames();
}

bool FDatasmithVREDImporter::CheckNodeType(const TSharedPtr<FDatasmithFBXSceneNode>& Node)
{
	if (Node->Mesh.IsValid() && Node->Camera.IsValid())
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Node '%ls' can't have a mesh and a camera at the same time!", *Node->Name);
		return false;
	}
	else if (Node->Mesh.IsValid() && Node->Light.IsValid())
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Node '%ls' can't have a mesh and a light at the same time!", *Node->Name);
		return false;
	}
	else if (Node->Light.IsValid() && Node->Camera.IsValid())
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Node '%ls' can't have a light and a camera at the same time!", *Node->Name);
		return false;
	}

	return true;
}

TSharedPtr<IDatasmithActorElement> FDatasmithVREDImporter::ConvertNode(const TSharedPtr<FDatasmithFBXSceneNode>& Node)
{
	FVector LocalPos = Node->LocalTransform.GetLocation();

	TSharedPtr<IDatasmithActorElement> ActorElement;

	// Check if node can be converted into a datasmith actor
	if (!CheckNodeType(Node))
	{
		return ActorElement;
	}

	if (Node->Mesh.IsValid())
	{
		TSharedPtr<FDatasmithFBXSceneMesh> ThisMesh = Node->Mesh;
		FName MeshName = FName(*ThisMesh->Name);

		TSharedPtr<IDatasmithMeshElement> CreatedMesh = nullptr;
		TSharedPtr<FDatasmithFBXSceneMesh>* FoundMesh = MeshNameToFBXMesh.Find(MeshName);
		if (FoundMesh && (*FoundMesh).IsValid())
		{
			// Meshes should all have unique names by now
			ensure(*FoundMesh == ThisMesh);
		}
		else
		{
			// Create a mesh
			MeshNameToFBXMesh.Add(MeshName, ThisMesh);
			CreatedMesh = FDatasmithSceneFactory::CreateMesh(*ThisMesh->Name);

			DatasmithScene->AddMesh(CreatedMesh);
		}

		TSharedPtr<IDatasmithMeshActorElement> MeshActorElement = FDatasmithSceneFactory::CreateMeshActor(*Node->Name);
		MeshActorElement->SetStaticMeshPathName(*ThisMesh->Name);

		// Assign materials to the actor
		for (int32 MaterialID = 0; MaterialID < Node->Materials.Num(); MaterialID++)
		{
			TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[MaterialID];
			TSharedPtr<IDatasmithBaseMaterialElement> MaterialElement = ConvertMaterial(Material);
			TSharedRef<IDatasmithMaterialIDElement> MaterialIDElement(FDatasmithSceneFactory::CreateMaterialId(MaterialElement->GetName()));
			MaterialIDElement->SetId(MaterialID);
			MeshActorElement->AddMaterialOverride(MaterialIDElement);

			// Also set the material directly on the mesh if this was the node that created it
			if (CreatedMesh)
			{
				CreatedMesh->SetMaterial(MaterialElement->GetName(), MaterialID);
			}
		}

		ActorElement = MeshActorElement;
	}
	else if (Node->Light.IsValid())
	{
		TSharedPtr<IDatasmithLightActorElement> LightActor;
		TSharedPtr<FDatasmithFBXSceneLight> Light = Node->Light;

		// Create the correct type of light and set some type-specific properties. Some others will be set just below this
		// switch, and yet others will be set on post-import, since they're not exposed on the IDatasmithLightActorElement hierarchy
		switch (Light->LightType)
		{
			case ELightType::Point:
			{
				TSharedRef<IDatasmithPointLightElement> PointLight = FDatasmithSceneFactory::CreatePointLight(*Node->Name);
				LightActor = PointLight;
				break;
			}
			case ELightType::Directional:
			{
				TSharedRef<IDatasmithDirectionalLightElement> DirLight = FDatasmithSceneFactory::CreateDirectionalLight(*Node->Name);
				LightActor = DirLight;
				break;
			}
			case ELightType::Spot:
			{
				TSharedRef<IDatasmithSpotLightElement> SpotLight = FDatasmithSceneFactory::CreateSpotLight(*Node->Name);
				SpotLight->SetInnerConeAngle(Light->ConeInnerAngle);
				SpotLight->SetOuterConeAngle(Light->ConeOuterAngle);
				LightActor = SpotLight;
				break;
			}
			case ELightType::Area:
			{
				TSharedRef<IDatasmithAreaLightElement> AreaLight = FDatasmithSceneFactory::CreateAreaLight(*Node->Name);
				AreaLight->SetInnerConeAngle(Light->ConeInnerAngle);
				AreaLight->SetOuterConeAngle(Light->ConeOuterAngle);
				AreaLight->SetLightShape(Light->AreaLightShape);

				if ( !Light->VisualizationVisible )
				{
					AreaLight->SetLightShape( EDatasmithLightShape::None );
				}

				// On VRED the user doesn't set the area light size, they just seem to spawn
				// with 20x20cm size by default, with a Scale3D of 100,100,100. To replicate this,
				// let's create a 0.2x0.2 cm size area light, also with a scale of 100,100,100
				AreaLight->SetWidth(0.2f);
				AreaLight->SetLength(0.2f);

				if (Light->UseIESProfile)
				{
					AreaLight->SetLightType(EDatasmithAreaLightType::IES_DEPRECATED);
				}
				else if (Light->AreaLightUseConeAngle)
				{
					AreaLight->SetLightType(EDatasmithAreaLightType::Spot);
				}
				else
				{
					AreaLight->SetLightType(EDatasmithAreaLightType::Point);
				}

				LightActor = AreaLight;
				break;
			}
			default:
			{
				TSharedRef<IDatasmithLightActorElement> DefaultLight = FDatasmithSceneFactory::CreateAreaLight(*Node->Name);
				LightActor = DefaultLight;
			}
		}

		//Set light units. Only IES-profile based lights seem to use lumens
		if (LightActor->IsA(EDatasmithElementType::PointLight | EDatasmithElementType::AreaLight | EDatasmithElementType::SpotLight))
		{
			IDatasmithPointLightElement* LightAsPointLight = static_cast<IDatasmithPointLightElement*>(LightActor.Get());
			if (LightAsPointLight)
			{
				if (Light->UseIESProfile)
				{
					LightAsPointLight->SetIntensityUnits(EDatasmithLightUnits::Lumens);
				}
				else
				{
					LightAsPointLight->SetIntensityUnits(EDatasmithLightUnits::Candelas);
				}
			}
		}

		LightActor->SetEnabled(Light->Enabled);
		LightActor->SetIntensity(Light->Intensity);
		LightActor->SetColor(Light->DiffuseColor);
		LightActor->SetTemperature(Light->Temperature);
		LightActor->SetUseTemperature(Light->UseTemperature);
		LightActor->SetUseIes(Light->UseIESProfile);
		if (Light->UseIESProfile && !Light->IESPath.IsEmpty())
		{
			// Create IES texture
			const FString BaseFilename = FPaths::GetBaseFilename(Light->IESPath);
			FString TextureName = FDatasmithUtils::SanitizeObjectName(BaseFilename + TEXT("_IES"));
			TSharedPtr<IDatasmithTextureElement> Texture = FDatasmithSceneFactory::CreateTexture(*TextureName);
			Texture->SetTextureMode(EDatasmithTextureMode::Ies);
			Texture->SetLabel(*BaseFilename);
			Texture->SetFile(*Light->IESPath);
			DatasmithScene->AddTexture(Texture);

			// Assign IES texture to light
			LightActor->SetIesTexturePathName(*TextureName);
		}

		ActorElement = LightActor;
	}
	else if (Node->Camera.IsValid())
	{
		TSharedPtr<FDatasmithFBXSceneCamera> Camera = Node->Camera;
		TSharedPtr<IDatasmithCameraActorElement> CameraActor = FDatasmithSceneFactory::CreateCameraActor(*Node->Name);

		CameraActor->SetFocalLength(Camera->FocalLength);
		CameraActor->SetFocusDistance(Camera->FocusDistance);
		CameraActor->SetSensorAspectRatio(Camera->SensorAspectRatio);
		CameraActor->SetSensorWidth(Camera->SensorWidth);

		//We will apply the roll value when splitting the camera node in the scene processor, since
		//we would affect the camera's children otherwise

		ActorElement = CameraActor;
	}
	else
	{
		// Create regular actor
		ActorElement = FDatasmithSceneFactory::CreateActor(*Node->Name);
	}

	if ( Node->Metadata.Num() > 0 )
	{
		TSharedPtr<IDatasmithMetaDataElement> Metadata = FDatasmithSceneFactory::CreateMetaData( ActorElement->GetName() );
		Metadata->SetAssociatedElement( ActorElement );
		DatasmithScene->AddMetaData( Metadata );

		for ( const TPair<FString, FString>& Pair : Node->Metadata )
		{
			const FString& Key = Pair.Key;
			const FString& Value = Pair.Value;

			TSharedPtr<IDatasmithKeyValueProperty> MetadataPropertyPtr = FDatasmithSceneFactory::CreateKeyValueProperty( *Key );
			MetadataPropertyPtr->SetPropertyType( EDatasmithKeyValuePropertyType::String );
			MetadataPropertyPtr->SetValue( *Value );
			Metadata->AddProperty( MetadataPropertyPtr );
		}
	}

	ActorElement->AddTag(*Node->OriginalName);
	ActorElement->AddTag(*FString::FromInt(Node->SplitNodeID));

	const FTransform& Transform = Node->GetWorldTransform();
	ActorElement->SetTranslation(Transform.GetTranslation());
	ActorElement->SetScale(Transform.GetScale3D());
	ActorElement->SetRotation(Transform.GetRotation());
	ActorElement->SetVisibility( Node->Visibility >= 0.5f );

#if !REVERSE_ATTACH_ORDER
	for (int32 Index = 0; Index < Node->Children.Num(); Index++)
#else
	for (int32 Index = Node->Children.Num() - 1; Index >= 0; Index--)
#endif
	{
		TSharedPtr<FDatasmithFBXSceneNode>& ChildNode = Node->Children[Index];
		TSharedPtr<IDatasmithActorElement> ChildNodeActor = ConvertNode(ChildNode);

		if (ChildNodeActor.IsValid())
		{
			ActorElement->AddChild(ChildNodeActor);
		}
	}

	return ActorElement;
}

void AddBoolProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, bool Value)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Bool);
	MaterialProperty->SetValue(Value ? TEXT("True") : TEXT("False"));
	Element->AddProperty(MaterialProperty);
}

void AddColorProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, const FVector4& Value)
{
	if (Value.ContainsNaN())
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Ignored vector parameter '%ls' of material '%ls' for being a NaN/infinite", *PropertyName, Element->GetName());
		return;
	}

	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Color);
	FLinearColor Color(Value.X, Value.Y, Value.Z, Value.W);
	MaterialProperty->SetValue(*Color.ToString());
	Element->AddProperty(MaterialProperty);
}

void AddFloatProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, float Value)
{
	if (!FMath::IsFinite(Value))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Ignored vector parameter '%ls' of material '%ls' for being a NaN/infinite", *PropertyName, Element->GetName());
		return;
	}

	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Float);
	MaterialProperty->SetValue(*FString::SanitizeFloat(Value));
	Element->AddProperty(MaterialProperty);
}

void AddStringProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, const FString& Value)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::String);
	MaterialProperty->SetValue(*Value);
	Element->AddProperty(MaterialProperty);
}

void AddTextureProperty(IDatasmithMaterialInstanceElement* Element, const FString& PropertyName, const FString& Path)
{
	TSharedPtr<IDatasmithKeyValueProperty> MaterialProperty = FDatasmithSceneFactory::CreateKeyValueProperty(*PropertyName);
	MaterialProperty->SetPropertyType(EDatasmithKeyValuePropertyType::Texture);
	MaterialProperty->SetValue(*Path);
	Element->AddProperty(MaterialProperty);
}

// Pack all "texture properties". These are really material properties, but we'll use these to help
// map the texture correctly. Datasmith will bind these values to the material instance on creation and it will
// do that by property name, so make sure the names match the ones on the reference materials
void AddTextureMappingProperties(IDatasmithMaterialInstanceElement* Element, const FString& TexHandle, const FDatasmithFBXSceneMaterial::FTextureParams& Tex)
{
	AddTextureProperty(Element, TexHandle + TEXT("Path"), Tex.Path);
	AddBoolProperty(Element, TexHandle + TEXT("IsActive"), !Tex.Path.IsEmpty() && Tex.bEnabled);
	AddFloatProperty(Element, TexHandle + TEXT("RepeatMode"), 1.0f * (uint8)Tex.RepeatMode);
	AddColorProperty(Element, TexHandle + TEXT("Color"), Tex.Color);

	// Displacement textures would trigger the usage of tessellation shaders, where transforming to local space is not supported.
	// This means we can't use planar or triplanar mapping for displacement textures at the moment, as those require
	// using local space (VRED does planar/triplanar projection in object/local space)
	ETextureMapType MappingType = Tex.ProjectionType;
	if (TexHandle == TEXT("TexDisplacement") && MappingType != ETextureMapType::UV)
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Displacement textures are only supported with UV mapping. Texture '%ls' will use UV mapping instead", *Tex.Path);
		MappingType = ETextureMapType::UV;
	}

	// We need to break the projeciton enum into bools so that they can be used in static bools and switches and
	// lead to discrete materials. If we don't do this, all samplings will be done every time, which would also
	// disable the usage of displacement maps altogether as they are not supported with planar/triplanar mapping
	AddBoolProperty(Element, TexHandle + TEXT("ProjectionIsUV"), MappingType == ETextureMapType::UV);
	AddBoolProperty(Element, TexHandle + TEXT("ProjectionIsPlanar"), MappingType == ETextureMapType::Planar);

	// Send all parameters every time, as a material might have properties for multiple projection types
	// and the user can decide to switch them while in UnrealEditor already
	AddBoolProperty(Element, TexHandle + TEXT("LimitedProjectionSize"), Tex.Scale.Z != FLT_MAX);
	AddColorProperty(Element, TexHandle + TEXT("ProjectionCenter"), Tex.Translation);
	AddColorProperty(Element, TexHandle + TEXT("ProjectionOrientation"), Tex.Rotation);
	AddColorProperty(Element, TexHandle + TEXT("ProjectionSize"), Tex.Scale);
	AddColorProperty(Element, TexHandle + TEXT("TriplanarRotation"), Tex.TriplanarRotation);
	AddColorProperty(Element, TexHandle + TEXT("TriplanarOffsetU"), Tex.TriplanarOffsetU);
	AddColorProperty(Element, TexHandle + TEXT("TriplanarOffsetV"), Tex.TriplanarOffsetV);
	AddColorProperty(Element, TexHandle + TEXT("TriplanarRepeatU"), Tex.TriplanarRepeatU);
	AddColorProperty(Element, TexHandle + TEXT("TriplanarRepeatV"), Tex.TriplanarRepeatV);
	AddFloatProperty(Element, TexHandle + TEXT("TriplanarBlendBias"), Tex.TriplanarBlendBias);
	AddColorProperty(Element, TexHandle + TEXT("TriplanarTextureSize"), Tex.TriplanarTextureSize);

	// Texture-space transformations (common to all projection types)
	AddColorProperty(Element, TexHandle + TEXT("Offset"), Tex.Offset);
	AddColorProperty(Element, TexHandle + TEXT("Repeat"), Tex.Repeat);
	AddFloatProperty(Element, TexHandle + TEXT("Rotate"), Tex.Rotate);
}

TSharedPtr<IDatasmithTextureElement> CreateTextureElement(IDatasmithMaterialInstanceElement* Element, const FString& TexHandle, const FString& TexturePath)
{
	using TextureModeEntry = TPairInitializer<const FString&, const EDatasmithTextureMode&>;
	const static TMap<FString, EDatasmithTextureMode> TextureHandleToModes
	({
		TextureModeEntry(TEXT("TexDiffuse"), EDatasmithTextureMode::Diffuse),
		TextureModeEntry(TEXT("TexGlossy"), EDatasmithTextureMode::Specular),
		TextureModeEntry(TEXT("TexSpecular"), EDatasmithTextureMode::Specular),
		TextureModeEntry(TEXT("TexIncandescence"), EDatasmithTextureMode::Diffuse),
		TextureModeEntry(TEXT("TexBump"), EDatasmithTextureMode::Normal),
		TextureModeEntry(TEXT("TexTransparent"), EDatasmithTextureMode::Other),
		TextureModeEntry(TEXT("TexScatter"), EDatasmithTextureMode::Other),
		TextureModeEntry(TEXT("TexRoughness"), EDatasmithTextureMode::Other),
		TextureModeEntry(TEXT("TexFresnel"), EDatasmithTextureMode::Other),
		TextureModeEntry(TEXT("TexRotation"), EDatasmithTextureMode::Other),
		TextureModeEntry(TEXT("TexIndexOfRefraction"), EDatasmithTextureMode::Other),
		TextureModeEntry(TEXT("TexSpecularBump"), EDatasmithTextureMode::Specular)
	});

	EDatasmithTextureMode TextureMode = EDatasmithTextureMode::Other;
	if (const EDatasmithTextureMode* FoundMode = TextureHandleToModes.Find(TexHandle))
	{
		TextureMode = *FoundMode;
	}

	TSharedRef<IDatasmithTextureElement> DSTexture = FDatasmithSceneFactory::CreateTexture(*FPaths::GetBaseFilename(TexturePath));
	DSTexture->SetTextureMode(TextureMode);
	DSTexture->SetFile(*TexturePath);

	// We handle repeat modes with material functions applied directly to UVs, so we can import a texture
	// only once and allow different materials to use different repeat modes. Since Wrap is the most common,
	// on the material side we just "pass the UVs through" for the Wrap case, meaning they need to be enforced Wrap here
	DSTexture->SetTextureAddressX(EDatasmithTextureAddress::Wrap);
	DSTexture->SetTextureAddressY(EDatasmithTextureAddress::Wrap);

	return DSTexture;
}

namespace VREDImporterImpl
{
	// This will search for a texture with a matching filename first in Path, then in the Textures folder
	FString SearchForFile(FString Path, const TArray<FDirectoryPath>& TextureFolders)
	{
		// The expected behaviour should be that even if the path is correct, if we provide no textures folder
		// it shouldn't import any textures
		if (Path.IsEmpty() || TextureFolders.Num() == 0)
		{
			return FString();
		}

		FPaths::NormalizeFilename(Path);
		if (FPaths::FileExists(Path))
		{
			return Path;
		}

		FString CleanFilename = FPaths::GetCleanFilename(Path);

		for (const FDirectoryPath& TextureFolderDir : TextureFolders)
		{
			const FString& TextureFolder = TextureFolderDir.Path;

			FString InTextureFolder = FPaths::Combine(TextureFolder, CleanFilename);
			if (FPaths::FileExists(InTextureFolder))
			{
				return InTextureFolder;
			}

			// Search recursively inside texture folder
			TArray<FString> FoundFiles;
			IFileManager::Get().FindFilesRecursive(FoundFiles, *TextureFolder, *CleanFilename, true, false);
			if (FoundFiles.Num() > 0)
			{
				return FoundFiles[0];
			}
		}

		return FString();
	}

	/* Recursively propagate the visibility so that hidden parents hide their children, like in VRED */
	void PropagateVisibility( TSharedPtr<FDatasmithFBXSceneNode>& Node, bool bRollingVisibility = true )
	{
		bool bNewVisibility = ( Node->Visibility >= 0.5f ) && bRollingVisibility;
		Node->Visibility = bNewVisibility ? 1.0f : 0.0f;

		for ( TSharedPtr<FDatasmithFBXSceneNode>& Child : Node->Children )
		{
			PropagateVisibility( Child, bNewVisibility );
		}
	}
}

TSharedPtr<IDatasmithBaseMaterialElement> FDatasmithVREDImporter::ConvertMaterial(const TSharedPtr<FDatasmithFBXSceneMaterial>& Material)
{
	TSharedPtr<IDatasmithBaseMaterialElement>* OldMaterial = ImportedMaterials.Find(Material);
	if (OldMaterial != nullptr)
	{
		return *OldMaterial;
	}

	TSharedPtr<IDatasmithMaterialInstanceElement> MaterialElement = FDatasmithSceneFactory::CreateMaterialInstance(*Material->Name);
	ImportedMaterials.Add(Material, MaterialElement);

	// Base colors used for Chrome and BrushedMetal materials
	using MetalColorEntry = TPairInitializer<const int32&, const FVector4&>;
	const static TMap<int32, FVector4> MetalColors
	({															// From UnrealEditor docs  // Experimental
		MetalColorEntry( 0, FVector4(0.950f, 0.950f, 0.950f, 1.000f)),                     // Highly reflective
		MetalColorEntry( 1, FVector4(0.913f, 0.921f, 0.925f, 1.000f)), // Aluminium
		MetalColorEntry( 2, FVector4(0.300f, 0.300f, 0.280f, 1.000f)),                     // Amorphous Carbon
		MetalColorEntry( 3, FVector4(0.972f, 0.960f, 0.915f, 1.000f)), // Silver
		MetalColorEntry( 4, FVector4(1.000f, 0.766f, 0.336f, 1.000f)), // Gold
		MetalColorEntry( 5, FVector4(0.662f, 0.655f, 0.634f, 1.000f)), // Cobalt
		MetalColorEntry( 6, FVector4(0.955f, 0.637f, 0.538f, 1.000f)), // Copper
		MetalColorEntry( 7, FVector4(0.550f, 0.556f, 0.554f, 1.000f)), // Chromium
		MetalColorEntry( 8, FVector4(0.930f, 0.940f, 0.900f, 1.000f)),                    // Lithium
		MetalColorEntry( 9, FVector4(0.780f, 0.779f, 0.779f, 1.000f)),                    // Mercury
		MetalColorEntry(10, FVector4(0.660f, 0.609f, 0.526f, 1.000f)), // Nickel
		MetalColorEntry(11, FVector4(0.910f, 0.918f, 0.920f, 1.000f)),                    // Potassium
		MetalColorEntry(12, FVector4(0.672f, 0.637f, 0.585f, 1.000f)), // Platinum
		MetalColorEntry(13, FVector4(0.672f, 0.637f, 0.585f, 1.000f)),                    // Iridium
		MetalColorEntry(14, FVector4(0.500f, 0.510f, 0.525f, 1.000f)),                    // Silicon
		MetalColorEntry(15, FVector4(0.550f, 0.560f, 0.565f, 1.000f)),                    // Amorphous Silicon
		MetalColorEntry(16, FVector4(0.979f, 0.928f, 0.930f, 1.000f)),                    // Sodium
		MetalColorEntry(17, FVector4(0.925f, 0.921f, 0.905f, 1.000f)),                    // Rhodium
		MetalColorEntry(18, FVector4(0.925f, 0.835f, 0.757f, 1.000f)),                    // Tungsten
		MetalColorEntry(19, FVector4(0.945f, 0.894f, 0.780f, 1.000f)),                    // Vanadium
		MetalColorEntry(20, FVector4(0.560f, 0.570f, 0.580f, 1.000f))  // Iron
	});

	IDatasmithMaterialInstanceElement* El = MaterialElement.Get();

	if (Material->Type.Equals(TEXT("UGlassMaterial")))
	{
		if (FVector4* FoundInteriorColor = Material->VectorParams.Find(TEXT("InteriorColor")))
		{
			// If it has a non-default interior color it means the backface of the mesh is visible with
			// this color in VRED, so let's turn on two-sided mode so that we can see it
			if (!FoundInteriorColor->Equals(FVector4(1.0f, 1.0f, 1.0f, 1.0f)))
			{
				Material->BoolParams.Add(TEXT("UseTwoSided"), true);
			}
		}

		// We do this so that all glass materials are transparent by default, while also allowing the
		// UnrealEditor user to move the opacity back to 1.0f and have it not be transparent. This also helps to match
		// VRED's multiply blend mode behaviour for translucent materials
		if (float* OldOpacity = Material->ScalarParams.Find(TEXT("Opacity")))
		{
			Material->ScalarParams[TEXT("Opacity")] = (*OldOpacity)/2.0f;
		}
	}
	else if (Material->Type.Equals(TEXT("UReflectivePlasticMaterial")))
	{
		// A reflective plastic is almost exactly the same as a regular plastic with roughness 0
		// The only difference in VRED is that it seems as though reflective plastics display accurate
		// reflections, but all of our materials can handle reflection probes just the same
		Material->Type = TEXT("UPlasticMaterial");
		Material->ScalarParams.Add(TEXT("Roughness"), 0.0f);
	}
	else if (Material->Type.Equals(TEXT("UChromeMaterial")))
	{
		FVector4 BaseColor(0.950f, 0.950f, 0.950f, 1.000f);

		if (float* MetalTypeFloat = Material->ScalarParams.Find(TEXT("MetalType")))
		{
			const int32 MetalType = (int32)(*MetalTypeFloat + 0.5f);
			if (const FVector4* FoundColor = MetalColors.Find(MetalType))
			{
				BaseColor = *FoundColor;
			}
		}

		Material->VectorParams.Add(TEXT("BaseColor"), BaseColor);
	}
	else if (Material->Type.Equals(TEXT("UBrushedMetalMaterial")))
	{
		// This material type has a special type of planar/triplanar mapping.
		// We will force all textures to triplanar mapping for now (the default),
		// as the planar mapping modes don't even match regular planar mapping
		FVector4 TriplanarRotate(0.0f, 0.0f, 0.0f, 1.0f);
		if (const FVector4* FoundRotate = Material->VectorParams.Find(TEXT("TriplanarRotate")))
		{
			TriplanarRotate = *FoundRotate;
		}
		for (auto& Pair : Material->TextureParams)
		{
			FString& TexName = Pair.Key;
			FDatasmithFBXSceneMaterial::FTextureParams& Tex = Pair.Value;

			Tex.ProjectionType = ETextureMapType::Triplanar;
			Tex.TriplanarRotation = TriplanarRotate;
		}

		// For raytracing, VRED allows different roughness directions. We don't support
		// that in UnrealEditor, so let's just combine the roughness values like VRED does in its
		// viewport so that it's easier to manipulate in the material instance
		const float* RoughnessU = Material->ScalarParams.Find(TEXT("RoughnessU"));
		const float* RoughnessV = Material->ScalarParams.Find(TEXT("RoughnessV"));
		if (RoughnessU != nullptr && RoughnessV != nullptr)
		{
			Material->ScalarParams.Add(TEXT("Roughness"), FMath::Min(*RoughnessU, *RoughnessV));
			Material->ScalarParams.Remove(TEXT("RoughnessU"));
			Material->ScalarParams.Remove(TEXT("RoughnessV"));
		}
		else if (RoughnessU != nullptr)
		{
			Material->ScalarParams.Add(TEXT("Roughness"), *RoughnessU);
			Material->ScalarParams.Remove(TEXT("RoughnessU"));
		}
		else if (RoughnessV != nullptr)
		{
			Material->ScalarParams.Add(TEXT("Roughness"), *RoughnessV);
			Material->ScalarParams.Remove(TEXT("RoughnessV"));
		}

		// MetalType 0 allows a custom diffuse color, so we'll let it pass unaltered
		// For all other cases we want to place a color according to the metal type
		if (float* MetalTypeFloat = Material->ScalarParams.Find(TEXT("MetalType")))
		{
			const int32 MetalType = (int32)(*MetalTypeFloat + 0.5f);

			if (MetalType != 0)
			{
				// VRED disables the diffuse texture if we have a pre-set metal selected
				if (FDatasmithFBXSceneMaterial::FTextureParams* DiffuseTex = Material->TextureParams.Find(TEXT("TexDiffuse")))
				{
					DiffuseTex->bEnabled = false;
				}

				if (const FVector4* FoundColor = MetalColors.Find(MetalType))
				{
					Material->VectorParams.Add(TEXT("DiffuseColor"), *FoundColor);
				}
			}
		}
	}

	// Add properties
	AddStringProperty(MaterialElement.Get(), TEXT("Type"), Material->Type);

	for (const auto& Pair : Material->TextureParams)
	{
		const FString& TexName = Pair.Key;
		const FDatasmithFBXSceneMaterial::FTextureParams& Tex = Pair.Value;

		// Change first character to upper case
		FString TexHandle = TexName;
		TexHandle[0] = TexHandle.Left(1).ToUpper()[0];

		AddTextureMappingProperties(El, TexHandle, Tex);

		FString FoundTexturePath = VREDImporterImpl::SearchForFile(Tex.Path, ImportOptions->TextureDirs);
		if (FoundTexturePath.IsEmpty())
		{
			continue;
		}

		if (!CreatedTextureElementPaths.Contains(FoundTexturePath))
		{
			TSharedPtr<IDatasmithTextureElement> CreatedTexture = CreateTextureElement(El, TexHandle, FoundTexturePath);
			if (CreatedTexture.IsValid())
			{
				DatasmithScene->AddTexture(CreatedTexture);
				CreatedTextureElementPaths.Add(FoundTexturePath);
			}
		}
	}
	for (const auto& Pair : Material->BoolParams)
	{
		const FString& ParamName = Pair.Key;
		const bool bValue = Pair.Value;

		AddBoolProperty(El, ParamName, bValue);
	}
	for (const auto& Pair : Material->ScalarParams)
	{
		const FString& ParamName = Pair.Key;
		const float Value = Pair.Value;

		AddFloatProperty(El, ParamName, Value);
	}
	for (const auto& Pair : Material->VectorParams)
	{
		const FString& ParamName = Pair.Key;
		const FVector4& Value = Pair.Value;

		AddColorProperty(El, ParamName, Value);
	}

	return MaterialElement;
}

void PopulateTransformAnimation(IDatasmithTransformAnimationElement& TransformAnimation, EDatasmithTransformType AnimationType, const FDatasmithFBXSceneAnimCurve* CurveX, const FDatasmithFBXSceneAnimCurve* CurveY, const FDatasmithFBXSceneAnimCurve* CurveZ, double ScaleFactor, float Framerate)
{
	if (!CurveX && !CurveY && !CurveZ)
	{
		return;
	}

	FVector Conversion = FVector(1.0f, 1.0f, 1.0f);

	switch(AnimationType)
	{
	case EDatasmithTransformType::Translation:
	{
		Conversion = FVector(ScaleFactor, -ScaleFactor, ScaleFactor); // Flip Y due to LH->RH conversion
		break;
	}
	case EDatasmithTransformType::Rotation:
	{
		break;
	}
	case EDatasmithTransformType::Scale:
	{
		Conversion = FVector(ScaleFactor, ScaleFactor, ScaleFactor);
		break;
	}
	default:
	{
		return;
	}
	}

	FRichCurve RichCurves[(int32)EDatasmithFBXSceneAnimationCurveComponent::Num];
	float MinKey = FLT_MAX;
	float MaxKey = -FLT_MAX;

	EDatasmithTransformChannels Channels = TransformAnimation.GetEnabledTransformChannels();
	ETransformChannelComponents Components = FDatasmithAnimationUtils::GetChannelTypeComponents(Channels, AnimationType);

	// Note that each component may have a different number of keys

	if (CurveX && CurveX->Points.Num() > 0)
	{
		Components |= ETransformChannelComponents::X;

		TArray<FRichCurveKey>& KeysX = RichCurves[(int32)EDatasmithFBXSceneAnimationCurveComponent::X].Keys;
		KeysX.Reserve(CurveX->Points.Num());
		for(const FDatasmithFBXSceneAnimPoint& Point : CurveX->Points)
		{
			FRichCurveKey* CurvePoint = new (KeysX) FRichCurveKey;
			CurvePoint->InterpMode = Point.InterpolationMode;
			CurvePoint->TangentMode = Point.TangentMode;
			CurvePoint->Time = Point.Time;
			CurvePoint->Value = Point.Value;
			CurvePoint->ArriveTangent = Point.ArriveTangent;
			CurvePoint->LeaveTangent = Point.LeaveTangent;

			MinKey = FMath::Min(MinKey, CurvePoint->Time);
			MaxKey = FMath::Max(MaxKey, CurvePoint->Time);
		}
	}

	if (CurveY && CurveY->Points.Num() > 0)
	{
		Components |= ETransformChannelComponents::Y;

		TArray<FRichCurveKey>& KeysY = RichCurves[(int32)EDatasmithFBXSceneAnimationCurveComponent::Y].Keys;
		KeysY.Reserve(CurveY->Points.Num());
		for(const FDatasmithFBXSceneAnimPoint& Point : CurveY->Points)
		{
			FRichCurveKey* CurvePoint = new (KeysY) FRichCurveKey;
			CurvePoint->InterpMode = Point.InterpolationMode;
			CurvePoint->TangentMode = Point.TangentMode;
			CurvePoint->Time = Point.Time;
			CurvePoint->Value = Point.Value;
			CurvePoint->ArriveTangent = Point.ArriveTangent;
			CurvePoint->LeaveTangent = Point.LeaveTangent;

			MinKey = FMath::Min(MinKey, CurvePoint->Time);
			MaxKey = FMath::Max(MaxKey, CurvePoint->Time);
		}
	}

	if (CurveZ && CurveZ->Points.Num() > 0)
	{
		Components |= ETransformChannelComponents::Z;

		TArray<FRichCurveKey>& KeysZ = RichCurves[(int32)EDatasmithFBXSceneAnimationCurveComponent::Z].Keys;
		KeysZ.Reserve(CurveZ->Points.Num());
		for(const FDatasmithFBXSceneAnimPoint& Point : CurveZ->Points)
		{
			FRichCurveKey* CurvePoint = new (KeysZ) FRichCurveKey;
			CurvePoint->InterpMode = Point.InterpolationMode;
			CurvePoint->TangentMode = Point.TangentMode;
			CurvePoint->Time = Point.Time;
			CurvePoint->Value = Point.Value;
			CurvePoint->ArriveTangent = Point.ArriveTangent;
			CurvePoint->LeaveTangent = Point.LeaveTangent;

			MinKey = FMath::Min(MinKey, CurvePoint->Time);
			MaxKey = FMath::Max(MaxKey, CurvePoint->Time);
		}
	}

	TransformAnimation.SetEnabledTransformChannels(Channels | FDatasmithAnimationUtils::SetChannelTypeComponents(Components, AnimationType));

	FFrameRate FrameRate = FFrameRate(static_cast<uint32>(Framerate + 0.5f), 1);
	FFrameNumber StartFrame = FrameRate.AsFrameNumber(MinKey);
	FFrameNumber EndFrame = FrameRate.AsFrameTime(MaxKey).CeilToFrame();

	// We go to EndFrame.Value+1 here so that if its a 2 second animation at 30fps, frame 60 belongs
	// to the actual animation, as opposed to being range [0, 59]. I don't think this is the standard
	// for UnrealEditor, but it is exactly how it works in VRED and it also guarantees that the animation will
	// actually complete within its range, which is necessary in order to play it as a subsequence
	// to its completion
	for (int32 Frame = StartFrame.Value; Frame <= EndFrame.Value + 1; ++Frame)
	{
		float TimeSeconds = FrameRate.AsSeconds(Frame);

		float XVal = RichCurves[(int32)EDatasmithFBXSceneAnimationCurveComponent::X].Eval(TimeSeconds);
		float YVal = RichCurves[(int32)EDatasmithFBXSceneAnimationCurveComponent::Y].Eval(TimeSeconds);
		float ZVal = RichCurves[(int32)EDatasmithFBXSceneAnimationCurveComponent::Z].Eval(TimeSeconds);

		// It doesn't matter what we place there really, as the channel will be disabled
		if (!CurveX)
		{
			XVal = MAX_flt;
		}
		if (!CurveY)
		{
			YVal = MAX_flt;
		}
		if (!CurveZ)
		{
			ZVal = MAX_flt;
		}

		FVector Val = FVector(XVal, YVal, ZVal);
		FDatasmithTransformFrameInfo FrameInfo = FDatasmithTransformFrameInfo(Frame, Val);
		TransformAnimation.AddFrame(AnimationType, FrameInfo);

		//UE_LOGF(LogTemp, Warning, "Adding frame %d of type %u at %f, vec: (%f, %f, %f)", Frame, (uint8)AnimationType, TimeSeconds, Val.X, Val.Y, Val.Z);
	}
}

void PopulateVisibilityAnimation(IDatasmithVisibilityAnimationElement& VisibilityAnimation, const FDatasmithFBXSceneAnimCurve* Curve, float Framerate)
{
	if (Curve == nullptr || Curve->Points.Num() == 0)
	{
		return;
	}

	FRichCurve RichCurve;
	float MinKey = FLT_MAX;
	float MaxKey = -FLT_MAX;

	TArray<FRichCurveKey>& Keys = RichCurve.Keys;
	Keys.Reserve(Curve->Points.Num());
	for(const FDatasmithFBXSceneAnimPoint& Point : Curve->Points)
	{
		FRichCurveKey* CurvePoint = new (Keys) FRichCurveKey;
		CurvePoint->InterpMode = Point.InterpolationMode;
		CurvePoint->TangentMode = Point.TangentMode;
		CurvePoint->Time = Point.Time;
		CurvePoint->Value = Point.Value;
		CurvePoint->ArriveTangent = Point.ArriveTangent;
		CurvePoint->LeaveTangent = Point.LeaveTangent;

		MinKey = FMath::Min(MinKey, CurvePoint->Time);
		MaxKey = FMath::Max(MaxKey, CurvePoint->Time);
	}

	FFrameRate FrameRate = FFrameRate(static_cast<uint32>(Framerate + 0.5f), 1);
	FFrameNumber StartFrame = FrameRate.AsFrameNumber(MinKey);
	FFrameNumber EndFrame = FrameRate.AsFrameTime(MaxKey).CeilToFrame();

	// We go to EndFrame.Value+1 here so that if its a 2 second animation at 30fps, frame 60 belongs
	// to the actual animation, as opposed to being frame [0, 59]. I don't think this is the standard
	// for UnrealEditor, but it is exactly how it works in VRED and it also guarantees that the animation will
	// actually complete within its range, which is necessary in order to play it as a subsequence
	// to its completion
	for (int32 Frame = StartFrame.Value; Frame <= EndFrame.Value + 1; ++Frame)
	{
		float TimeSeconds = FrameRate.AsSeconds(Frame);
		bool bVisible = RichCurve.Eval(TimeSeconds) >= 1.0f;

		FDatasmithVisibilityFrameInfo FrameInfo = FDatasmithVisibilityFrameInfo(Frame, bVisible);
		VisibilityAnimation.AddFrame(FrameInfo);
	}
}

using CurveMap = TMap<EDatasmithFBXSceneAnimationCurveType, TArray<const FDatasmithFBXSceneAnimCurve*>>;
void FetchCurvesForCurveType(CurveMap& Map, EDatasmithFBXSceneAnimationCurveType CurveType, const FDatasmithFBXSceneAnimCurve*& OutCurveX, const FDatasmithFBXSceneAnimCurve*& OutCurveY, const FDatasmithFBXSceneAnimCurve*& OutCurveZ)
{
	OutCurveX = nullptr;
	OutCurveY = nullptr;
	OutCurveZ = nullptr;

	if (TArray<const FDatasmithFBXSceneAnimCurve*>* Curves = Map.Find(CurveType))
	{
		for (const FDatasmithFBXSceneAnimCurve* Curve : *Curves)
		{
			if (Curve == nullptr || Curve->Points.Num() == 0)
			{
				continue;
			}

			if (OutCurveX == nullptr && Curve->Component == EDatasmithFBXSceneAnimationCurveComponent::X)
			{
				OutCurveX = Curve;
			}
			else if (OutCurveY == nullptr && Curve->Component == EDatasmithFBXSceneAnimationCurveComponent::Y)
			{
				OutCurveY = Curve;
			}
			else if (OutCurveZ == nullptr && Curve->Component == EDatasmithFBXSceneAnimationCurveComponent::Z)
			{
				OutCurveZ = Curve;
			}
		}
	}
}

TSharedPtr<IDatasmithLevelSequenceElement> FDatasmithVREDImporter::ConvertAnimBlock(const FCombinedAnimBlock& CombinedBlock)
{
	if (CombinedBlock.NodeNameToBlock.Num() < 1)
	{
		return nullptr;
	}

	FString BlockName = CombinedBlock.NodeNameToBlock.CreateConstIterator().Value()->Name;

	TSharedRef<IDatasmithLevelSequenceElement> SequenceElement = FDatasmithSceneFactory::CreateLevelSequence(*BlockName);
	SequenceElement->SetFrameRate(IntermediateScene->BaseTime);

	for (const auto& Pair : CombinedBlock.NodeNameToBlock)
	{
		TSharedPtr<IDatasmithVisibilityAnimationElement> VisibilityAnimation = nullptr;
		TSharedPtr<IDatasmithTransformAnimationElement> TransformAnimation = nullptr;

		const FString& NodeName = Pair.Key;
		FDatasmithFBXSceneAnimBlock* AnimBlock = Pair.Value;

		// An anim block may have something like 2 transform curves, 3 scale curves, a rotation curve and a visibility curve
		CurveMap CurvesPerType;
		for (const FDatasmithFBXSceneAnimCurve& Curve : AnimBlock->Curves)
		{
			TArray<const FDatasmithFBXSceneAnimCurve*>& Curves = CurvesPerType.FindOrAdd(Curve.Type);
			Curves.Add(&Curve);
		}

		if (CurvesPerType.Contains(EDatasmithFBXSceneAnimationCurveType::Translation))
		{
			const FDatasmithFBXSceneAnimCurve* CurveX;
			const FDatasmithFBXSceneAnimCurve* CurveY;
			const FDatasmithFBXSceneAnimCurve* CurveZ;
			FetchCurvesForCurveType(CurvesPerType, EDatasmithFBXSceneAnimationCurveType::Translation, CurveX, CurveY, CurveZ);

			if (CurveX || CurveY || CurveZ)
			{
				if (!TransformAnimation.IsValid())
				{
					TransformAnimation = FDatasmithSceneFactory::CreateTransformAnimation(*NodeName);
					TransformAnimation->SetEnabledTransformChannels(EDatasmithTransformChannels::None);
				}

				if (TransformAnimation.IsValid())
				{
					PopulateTransformAnimation(TransformAnimation.ToSharedRef().Get(), EDatasmithTransformType::Translation, CurveX, CurveY, CurveZ, IntermediateScene->ScaleFactor, IntermediateScene->BaseTime);
				}
			}
		}

		if (CurvesPerType.Contains(EDatasmithFBXSceneAnimationCurveType::Scale))
		{
			const FDatasmithFBXSceneAnimCurve* CurveX;
			const FDatasmithFBXSceneAnimCurve* CurveY;
			const FDatasmithFBXSceneAnimCurve* CurveZ;
			FetchCurvesForCurveType(CurvesPerType, EDatasmithFBXSceneAnimationCurveType::Scale, CurveX, CurveY, CurveZ);

			if (CurveX || CurveY || CurveZ)
			{
				if (!TransformAnimation.IsValid())
				{
					TransformAnimation = FDatasmithSceneFactory::CreateTransformAnimation(*NodeName);
					TransformAnimation->SetEnabledTransformChannels(EDatasmithTransformChannels::None);
				}

				if (TransformAnimation.IsValid())
				{
					PopulateTransformAnimation(TransformAnimation.ToSharedRef().Get(), EDatasmithTransformType::Scale, CurveX, CurveY, CurveZ, IntermediateScene->ScaleFactor, IntermediateScene->BaseTime);
				}
			}
		}

		if (CurvesPerType.Contains(EDatasmithFBXSceneAnimationCurveType::Rotation))
		{
			const FDatasmithFBXSceneAnimCurve* CurveX;
			const FDatasmithFBXSceneAnimCurve* CurveY;
			const FDatasmithFBXSceneAnimCurve* CurveZ;
			FetchCurvesForCurveType(CurvesPerType, EDatasmithFBXSceneAnimationCurveType::Rotation, CurveX, CurveY, CurveZ);

			if (CurveX || CurveY || CurveZ)
			{
				if (!TransformAnimation.IsValid())
				{
					TransformAnimation = FDatasmithSceneFactory::CreateTransformAnimation(*NodeName);
					TransformAnimation->SetEnabledTransformChannels(EDatasmithTransformChannels::None);
				}

				if (TransformAnimation.IsValid())
				{
					PopulateTransformAnimation(TransformAnimation.ToSharedRef().Get(), EDatasmithTransformType::Rotation, CurveX, CurveY, CurveZ, IntermediateScene->ScaleFactor, IntermediateScene->BaseTime);
				}
			}
		}

		if (CurvesPerType.Contains(EDatasmithFBXSceneAnimationCurveType::Visible))
		{
			TArray<const FDatasmithFBXSceneAnimCurve*>& Curves = CurvesPerType[EDatasmithFBXSceneAnimationCurveType::Visible];
			if (Curves.Num() > 1)
			{
				UE_LOGF(LogDatasmithVREDImport, Error, "AnimBlock for node '%ls' has more than one Visibility curve. The first one will be used.", *NodeName);
			}

			if (Curves.Num() > 0 && Curves[0] != nullptr && Curves[0]->Points.Num() > 0)
			{
				if (!VisibilityAnimation.IsValid())
				{
					VisibilityAnimation = FDatasmithSceneFactory::CreateVisibilityAnimation(*NodeName);
				}

				if (VisibilityAnimation.IsValid())
				{
					VisibilityAnimation->SetPropagateToChildren(true);
					PopulateVisibilityAnimation(VisibilityAnimation.ToSharedRef().Get(), Curves[0], IntermediateScene->BaseTime);
				}
			}
		}

		if (TransformAnimation.IsValid())
		{
			TransformAnimation->SetCompletionMode(EDatasmithCompletionMode::KeepState);
			SequenceElement->AddAnimation(TransformAnimation.ToSharedRef());
		}

		if (VisibilityAnimation.IsValid())
		{
			VisibilityAnimation->SetCompletionMode(EDatasmithCompletionMode::KeepState);
			SequenceElement->AddAnimation(VisibilityAnimation.ToSharedRef());
		}
	}

	return SequenceElement;
}

void ConvertAnimClips(TArray<FDatasmithFBXSceneAnimClip>& AnimClips, TMap<FString, FImportedAnim>& ImportedAnims, TArray<TSharedPtr<IDatasmithLevelSequenceElement>>& OutCreatedClips, float Framerate)
{
	TArray<FDatasmithFBXSceneAnimClip> ClipsRemaining = AnimClips;

	while (ClipsRemaining.Num() > 0)
	{
		int32 NumClipsParsedThisLoop = 0;
		for (int32 ClipIndex = ClipsRemaining.Num()-1; ClipIndex >= 0; ClipIndex--)
		{
			FDatasmithFBXSceneAnimClip& Clip = ClipsRemaining[ClipIndex];

			// Check if we can parse this clip
			bool bCanParse = true;
			TArray<FString> MissingParsedUsages;
			for (FDatasmithFBXSceneAnimUsage& Usage : Clip.AnimUsages)
			{
				if (!ImportedAnims.Contains(Usage.AnimName))
				{
					bCanParse = false;
					MissingParsedUsages.Add(Usage.AnimName);
					//break;
				}
			}
			if (!bCanParse)
			{
				UE_LOGF(LogDatasmithVREDImport, Verbose, "Postponed parsing AnimClip '%ls'. It still needs these usages:", *Clip.Name);
				for (const FString& Usage : MissingParsedUsages)
				{
					UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "\t%ls", *Usage);
				}

				UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "This is what has been imported so far:");
				for (const auto& Pair : ImportedAnims)
				{
					UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "\t%ls", *Pair.Key);
				}
				continue;
			}

			UE_LOGF(LogDatasmithVREDImport, Verbose, "Parsing AnimClip '%ls'", *Clip.Name);

			// Create a IDatasmithLevelSequenceElement and populate it with pointers to the target subsequence elements
			TSharedPtr<IDatasmithLevelSequenceElement> SequenceElement = FDatasmithSceneFactory::CreateLevelSequence(*Clip.Name);
			SequenceElement->SetFrameRate(Framerate);

			FFrameRate FrameRate = FFrameRate(static_cast<uint32>(Framerate + 0.5f), 1);

			float MinStartTime = FLT_MAX;
			float MaxEndTime = -FLT_MAX;

			for (FDatasmithFBXSceneAnimUsage& Usage : Clip.AnimUsages)
			{
				if (FImportedAnim* FoundAnim = ImportedAnims.Find(Usage.AnimName))
				{
					TSharedPtr<IDatasmithSubsequenceAnimationElement> SubsequenceAnimation = FDatasmithSceneFactory::CreateSubsequenceAnimation(*Usage.AnimName);
					SubsequenceAnimation->SetSubsequence(FoundAnim->ImportedSequence);

					float TargetDurationSeconds = Usage.EndTime - Usage.StartTime;
					float OriginalDurationSeconds = FoundAnim->OriginalEndSeconds - FoundAnim->OriginalStartSeconds;
					float TimeScale = OriginalDurationSeconds / TargetDurationSeconds;

					MinStartTime = FMath::Min(MinStartTime, Usage.StartTime);
					MaxEndTime = FMath::Max(MaxEndTime, Usage.EndTime);

					FFrameNumber StartFrame = FrameRate.AsFrameNumber(Usage.StartTime);

					// RoundToFrame here because we will add one extra frame anyway
					// If the last subframe is larger than 0.5, it seems the sequencer will "play it",
					// so flooring and adding our +1 frame would not be enough. Additionally, doing a ceil might
					// cause some floating point errors to generate an extra, unnecessary, additional frame
					FFrameNumber EndFrame = FrameRate.AsFrameTime(Usage.EndTime).RoundToFrame();

					// Note how we add +1 here and not to MaxEndTime: This ensures every subsequence animation
					// we create will have one extra frame than what Usage.EndTime says (which is necessary because
					// we add 1 extra frame when populating transform/visibility animations), but that these
					// +1 will not pile up as we progressively nest subsequences
					SubsequenceAnimation->SetDuration((EndFrame - StartFrame).Value + 1);
					SubsequenceAnimation->SetStartTime(StartFrame);
					SubsequenceAnimation->SetTimeScale(TimeScale);
					SubsequenceAnimation->SetCompletionMode(EDatasmithCompletionMode::KeepState);

					UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "\tParsed subsequence animation. TargetClip: '%ls', StartTime: %d, TimeScale: %f, OrigStart: %f, OrigEnd: %f, UsageStart: %f, UsageEnd: %f", *Usage.AnimName, StartFrame.Value, TimeScale, FoundAnim->OriginalStartSeconds, FoundAnim->OriginalEndSeconds, Usage.StartTime, Usage.EndTime);
					SequenceElement->AddAnimation(SubsequenceAnimation.ToSharedRef());
				}
			}

			UE_LOGF(LogDatasmithVREDImport, Verbose, "Parsed AnimClip '%ls', which depends on the following clips:", *Clip.Name);
			for (FDatasmithFBXSceneAnimUsage& Usage : Clip.AnimUsages)
			{
				UE_LOGF(LogDatasmithVREDImport, Verbose, "\t%ls", *Usage.AnimName);
			}

			// Add our created IDatasmithLevelSequenceElement to our output and our dict, so that it can
			// be used to parse nested clips. We make sure all AnimNodes have unique names in VRED before
			// exporting
			ensure(!ImportedAnims.Contains(Clip.Name));

			FImportedAnim NewImportedAnim;
			NewImportedAnim.Name = Clip.Name;
			NewImportedAnim.ImportedSequence = SequenceElement;
			NewImportedAnim.OriginalStartSeconds = MinStartTime;
			NewImportedAnim.OriginalEndSeconds = MaxEndTime;
			ImportedAnims.Add(NewImportedAnim.Name, NewImportedAnim);
			OutCreatedClips.Add(SequenceElement);

			// Update control flow
			NumClipsParsedThisLoop++;
			ClipsRemaining.RemoveAt(ClipIndex);
		}

		// If this clip network is well-behaved, we should be able to parse at least one clip per pass
		if (NumClipsParsedThisLoop == 0)
		{
			FString ErrorMessage = TEXT("Ran into a lock trying to parse AnimClips. These are the remaining clips:\n");
			for (FDatasmithFBXSceneAnimClip& Clip : ClipsRemaining)
			{
				ErrorMessage += FString(TEXT("\t")) + Clip.Name + TEXT("\n");
			}
			ErrorMessage.RemoveFromEnd(TEXT("\n"));
			UE_LOGF(LogDatasmithVREDImport, Error, "%ls", *ErrorMessage);
			break;
		}
	}
}

struct FNameDuplicateFinder
{
	TMap<FString, int32> NodeNames;
	TMap<FString, int32> MeshNames;
	TMap<FString, int32> MaterialNames;

	TSet< TSharedPtr<FDatasmithFBXSceneMesh> > ProcessedMeshes;
	TSet< TSharedPtr<FDatasmithFBXSceneMaterial> > ProcessedMaterials;

	void MakeUniqueName(FString& Name, TMap<FString, int32>& NameList)
	{
		// We're using lowercase name value to make NameList case-insensitive. These names
		// should be case-insensitive because uasset file names directly depends on them.
		FString LowercaseName = Name.ToLower();
		int32* LastValue = NameList.Find(LowercaseName);
		if (LastValue == nullptr)
		{
			// Simplest case: name is not yet used
			NameList.Add(LowercaseName, 0);
			return;
		}

		// Append a numeric suffix
		int32 NameIndex = *LastValue + 1;
		FString NewName;
		do
		{
			NewName = FString::Printf(TEXT("%s%s%d"), *Name, UNIQUE_NAME_SUFFIX, NameIndex);
		} while (NameList.Contains(NewName));

		// Remember the new name
		*LastValue = NameIndex;
		NameList.Add(NewName.ToLower());
		Name = NewName;
	}

	void ResolveDuplicatedObjectNamesRecursive(TSharedPtr<FDatasmithFBXSceneNode>& Node)
	{
		// Process node name
		MakeUniqueName(Node->Name, NodeNames);

		// Process mesh name
		TSharedPtr<FDatasmithFBXSceneMesh>& Mesh = Node->Mesh;
		if (Mesh.IsValid() && !ProcessedMeshes.Contains(Mesh))
		{
			if (Mesh->Name.Len() > MAX_MESH_NAME_LENGTH)
			{
				// Truncate the mesh name if it is too long
				FString NewName = Mesh->Name.Left(MAX_MESH_NAME_LENGTH - 3) + TEXT("_tr");
				UE_LOGF(LogDatasmithVREDImport, Warning, "Mesh name '%ls' is too long, renaming to '%ls'", *Mesh->Name, *NewName);
				Mesh->Name = NewName;
			}

			MakeUniqueName(Mesh->Name, MeshNames);
			ProcessedMeshes.Add(Mesh);
		}

		// Process material names
		for (int32 MaterialIndex = 0; MaterialIndex < Node->Materials.Num(); MaterialIndex++)
		{
			TSharedPtr<FDatasmithFBXSceneMaterial>& Material = Node->Materials[MaterialIndex];
			if (Material.IsValid() && !ProcessedMaterials.Contains(Material))
			{
				MakeUniqueName(Material->Name, MaterialNames);
				ProcessedMaterials.Add(Material);
			}
		}

		for (int32 ChildIndex = 0; ChildIndex < Node->Children.Num(); ChildIndex++)
		{
			ResolveDuplicatedObjectNamesRecursive(Node->Children[ChildIndex]);
		}
	}
};

// AnimBlocks are supposed to have unique names, except when they originated from a
// node being split, in which case all generated blocks will have the same names. This
// function groups those split blocks into CombinedAnimBlock objects
TArray<FCombinedAnimBlock> CombineBlocks(TArray<FDatasmithFBXSceneAnimNode>& InNodes)
{
	struct FBlockAndParent
	{
		FDatasmithFBXSceneAnimBlock* Block;
		FDatasmithFBXSceneAnimNode* Parent;
	};

	TMap<FString, TArray<FBlockAndParent>> BlocksWithSameName;
	for (FDatasmithFBXSceneAnimNode& Node : InNodes)
	{
		for (FDatasmithFBXSceneAnimBlock& Block : Node.Blocks)
		{
			TArray<FBlockAndParent>& BlocksWithThisName = BlocksWithSameName.FindOrAdd(Block.Name);

			FBlockAndParent* NewBlockAndParent = new(BlocksWithThisName) FBlockAndParent;
			NewBlockAndParent->Block = &Block;
			NewBlockAndParent->Parent = &Node;
		}
	}

	TArray<FCombinedAnimBlock> Result;
	for (auto& Pair : BlocksWithSameName)
	{
		FString& BlockName = Pair.Key;
		TArray<FBlockAndParent>& Blocks = Pair.Value;

		FCombinedAnimBlock* NewCombinedBlock = new(Result) FCombinedAnimBlock;
		for (const FBlockAndParent& BnP : Blocks)
		{
			NewCombinedBlock->NodeNameToBlock.Add(BnP.Parent->Name, BnP.Block);
		}
	}

	return Result;
}

bool FDatasmithVREDImporter::SendSceneToDatasmith()
{
	if (!IntermediateScene->RootNode.IsValid())
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "FBX Scene root is invalid!");
		return false;
	}
	// Ensure nodes, meshes and materials have unique names
	FNameDuplicateFinder NameDupContext;
	NameDupContext.ResolveDuplicatedObjectNamesRecursive(IntermediateScene->RootNode);

	VREDImporterImpl::PropagateVisibility(IntermediateScene->RootNode);

	// Perform conversion
	TSharedPtr<IDatasmithActorElement> NodeActor = ConvertNode(IntermediateScene->RootNode);

	if (NodeActor.IsValid())
	{
		// We need the root node as that is what carries the scaling factor conversion
		DatasmithScene->AddActor(NodeActor);

		for (TSharedPtr<FDatasmithFBXSceneMaterial>& Material : IntermediateScene->Materials)
		{
			TSharedPtr<IDatasmithBaseMaterialElement> ConvertedMaterial = ConvertMaterial(Material);
			DatasmithScene->AddMaterial(ConvertedMaterial);
		}

		TMap<FString, FImportedAnim> ImportedAnimElements;

		// During scene processing, we might split a node with rotation/scaling pivots into multiple nodes
		// and each receives part of the curves. In that case we will create blocks with the same name for all
		// of them. Blocks have unique names otherwise, so this means they should be combined
		// Over here we build this structure so that we can create a single SequenceElement from it, allowing
		// clips to not really care if nodes were split or not
		TArray<FCombinedAnimBlock> CombinedBlocks = CombineBlocks(IntermediateScene->AnimNodes);

		for (FCombinedAnimBlock& CombinedBlock : CombinedBlocks) //-V1078
		{
			TSharedPtr<IDatasmithLevelSequenceElement> ConvertedBlock = ConvertAnimBlock(CombinedBlock);
			if (ConvertedBlock.IsValid())
			{
				DatasmithScene->AddLevelSequence(ConvertedBlock.ToSharedRef());

				FImportedAnim NewImportedAnim;
				NewImportedAnim.Name = FString(ConvertedBlock->GetName());
				NewImportedAnim.ImportedSequence = ConvertedBlock;
				NewImportedAnim.OriginalStartSeconds = CombinedBlock.GetStartSeconds();
				NewImportedAnim.OriginalEndSeconds = CombinedBlock.GetEndSeconds();
				ImportedAnimElements.Add(NewImportedAnim.Name, NewImportedAnim);
			}
		}

		TArray<TSharedPtr<IDatasmithLevelSequenceElement>> CreatedClips;
		ConvertAnimClips(ParsedAnimClips, ImportedAnimElements, CreatedClips, IntermediateScene->BaseTime);

		for (const TSharedPtr<IDatasmithLevelSequenceElement>& CreatedClip : CreatedClips)
		{
			if (CreatedClip.IsValid())
			{
				UE_LOGF(LogDatasmithVREDImport, Verbose, "Added clip level sequence '%ls' to the scene", CreatedClip->GetName());
				DatasmithScene->AddLevelSequence(CreatedClip.ToSharedRef());
			}
		}

		FActorMap ImportedActorsByOriginalName;
		FMaterialMap ImportedMaterialsByName;
		BuildAssetMaps(DatasmithScene, ImportedActorsByOriginalName, ImportedMaterialsByName);

		if (ImportOptions->bImportVar)
		{
			TSharedPtr<IDatasmithLevelVariantSetsElement> LevelVariantSets = FVREDVariantConverter::ConvertVariants(ParsedVariants, ImportedActorsByOriginalName, ImportedMaterialsByName);
			if (LevelVariantSets.IsValid())
			{
				DatasmithScene->AddLevelVariantSets(LevelVariantSets.ToSharedRef());
			}
		}
	}
	else
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Root node '%ls' failed to convert!", *IntermediateScene->RootNode->Name);
		return false;
	}
	return true;
}

#undef LOCTEXT_NAMESPACE
