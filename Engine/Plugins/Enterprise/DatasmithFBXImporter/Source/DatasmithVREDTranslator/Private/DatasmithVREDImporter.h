// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFBXImporter.h"


struct FVREDCppVariant;

class IDatasmithActorElement;
class IDatasmithBaseMaterialElement;
class IDatasmithLevelSequenceElement;
class IDatasmithScene;
class UDatasmithVREDImportOptions;
struct FCombinedAnimBlock;
struct FDatasmithFBXSceneAnimClip;
struct FDatasmithFBXSceneLight;
struct FDatasmithFBXSceneMaterial;
struct FDatasmithFBXSceneNode;

class FDatasmithVREDImporter : public FDatasmithFBXImporter
{
public:
	FDatasmithVREDImporter(TSharedRef<IDatasmithScene>& OutScene, UDatasmithVREDImportOptions* InOptions);
	virtual ~FDatasmithVREDImporter();

	/** Updates the used import options to InOptions */
	void SetImportOptions(UDatasmithVREDImportOptions* InOptions);

	/** Open and load a scene file. */
	bool OpenFile(const FString& FilePath);

	/** Finalize import of the VRED scene into datasmith */
	bool SendSceneToDatasmith();

	/** Clean up any unused memory or data. */
	void UnloadScene();

private:
	/** Open and load a VRED .fbx scene file, placing its data in the context */
	bool ParseFbxFile(const FString& FBXPath);

	/** Open auxilliary .lights, .clips and/or .var files exported from VRED into the context */
	void ParseAuxFiles(const FString& FBXPath);

	/** Perform processing of internal scene data for better performance after import */
	void ProcessScene();

	/** Checks if the NodeType flag has a combination supported for this plugin */
	bool CheckNodeType(const TSharedPtr<FDatasmithFBXSceneNode>& Node);

	TSharedPtr<IDatasmithActorElement> ConvertNode(const TSharedPtr<FDatasmithFBXSceneNode>& Node);

	TSharedPtr<IDatasmithBaseMaterialElement> ConvertMaterial(const TSharedPtr<FDatasmithFBXSceneMaterial>& Material);

	/** Converts an anim block into IDatasmithLevelSequenceElement.	Each AnimBlock is a set of transform/visibility tracks.
	Note that if no AnimClips are present, these AnimBlocks are a completely arbitrary selection of tracks, as that
	is the best we can possibly do */
	TSharedPtr<IDatasmithLevelSequenceElement> ConvertAnimBlock(const FCombinedAnimBlock& CombinedBlock);

private:
	/** Output Datasmith scene */
	TSharedRef<IDatasmithScene> DatasmithScene;

	/** All options to configure importer */
	const UDatasmithVREDImportOptions* ImportOptions;

	// Used to prevent us from needlessly importing the same texture multiple times
	TSet<FString> CreatedTextureElementPaths;

	// Things parsed from aux files
	TArray<FDatasmithFBXSceneAnimClip> ParsedAnimClips;
	TArray<FDatasmithFBXSceneLight> ParsedLightsInfo;
	TArray<FDatasmithFBXSceneMaterial> ParsedMats;
	TArray<FVREDCppVariant> ParsedVariants;
};
