// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDImporterAuxFiles.h"

#include "DatasmithDefinitions.h"
#include "DatasmithFBXScene.h"
#include "DatasmithUtils.h"
#include "DatasmithVREDImportData.h"
#include "DatasmithVREDLog.h"

#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "XmlFile.h"
#include "XmlNode.h"

// Optimize this statement: for(const FXmlNode* Node = ParentNode->GetFirstChildNode();Node; Node = Node->GetNextNode())
// into this: for(const FXmlNode* Node: FXmlNodeChildren(ParentNode))
class FXmlNodeChildren
{
	const FXmlNode* Node;
public:
	class Iterator{
		const FXmlNode* Node;
	public:
		Iterator(const FXmlNode* Node): Node(Node) {}

		const Iterator& operator++()
		{
			Node = Node->GetNextNode();
			return *this;
		}

		bool operator!=(const Iterator& other)
		{
			return Node != other.Node;
		}

		const FXmlNode* operator*()
		{
			return Node;
		}
	};

	FXmlNodeChildren(const FXmlNode* Node): Node(Node) {}

	const Iterator begin(){
		return Iterator(Node->GetFirstChildNode());
	}
	const Iterator end(){
		return Iterator(nullptr);
	}
};

void StringToFloatArray(const FString& InString, TArray<float>& OutArray)
{
	FString strCopy = FString(InString);

	strCopy.RemoveSpacesInline();
	strCopy.RemoveFromStart(TEXT("["));
	strCopy.RemoveFromEnd(TEXT("]"));

	TArray<FString> floatStrs;
	strCopy.ParseIntoArray(floatStrs, TEXT(","));

	OutArray.Empty();
	OutArray.Reserve(floatStrs.Num());

	for (FString floatStr : floatStrs)
	{
		OutArray.Add(FCString::Atof(*floatStr));
	}
}

bool FloatArrayToTransform(const TArray<float>& Floats, FTransform& Transform)
{
	if ( Floats.Num() != 16 )
	{
		Transform.SetIdentity();
		return false;
	}

	FMatrix44f matrix = FMatrix44f();
	float (*matrixData)[4] = matrix.M;

	const float* floatsData = Floats.GetData();

	FMemory::Memcpy(matrixData[0], &floatsData[0], 4 * sizeof(float));
	FMemory::Memcpy(matrixData[1], &floatsData[4], 4 * sizeof(float));
	FMemory::Memcpy(matrixData[2], &floatsData[8], 4 * sizeof(float));
	FMemory::Memcpy(matrixData[3], &floatsData[12], 4 * sizeof(float));

	Transform.SetFromMatrix(FMatrix(matrix));

	return true;
}

void FixVREDXml(TArray<FString>& FileContentLines)
{
	for (FString& Line : FileContentLines)
	{
		// replace text contents of Name/name tag with spaces changed to "_" underscore
		// this fixes problem with FastXML that changes all sequences of spaces to single space
		// which led to mismatching node names referenced in XML with node names imported from
		// FBX(where all space sequences kept, having each space converted to "_")
		{
			FString NameTagOpen = "<Name>";
			FString NameTagClose = "</Name>";
			if (Line.TrimStart().StartsWith(NameTagOpen, ESearchCase::IgnoreCase))
			{
				int32 NameTagOpenIndex = Line.Find(NameTagOpen, ESearchCase::IgnoreCase);

				int32 NameTagCloseIndex = Line.Find(NameTagClose, ESearchCase::IgnoreCase, ESearchDir::FromEnd);

				if (NameTagOpenIndex >= 0 && NameTagOpenIndex < NameTagCloseIndex)
				{
					int32 NameTagOpenEndIndex = NameTagOpenIndex + NameTagOpen.Len();
					FString NameTextContent = Line.Mid(NameTagOpenEndIndex, NameTagCloseIndex - NameTagOpenEndIndex);
					Line = Line.Mid(0, NameTagOpenEndIndex) + NameTextContent.Replace(TEXT(" "), TEXT("_")) + Line.Mid(NameTagCloseIndex);
				}
			}
		}
	}
}

void ParseVariantSet(const FXmlNode* InVariantSetNode, FVREDCppVariant* OutVariantSet, const FString& VariantSetGroup)
{
	OutVariantSet->Name = InVariantSetNode->GetAttribute("name");
	OutVariantSet->Type = EVREDCppVariantType::VariantSet;
	OutVariantSet->VariantSet.VariantSetGroupName = VariantSetGroup;

	UE_LOGF(LogDatasmithVREDImport, Verbose, "VariantSet '%ls' imported within group '%ls'", *OutVariantSet->Name, *VariantSetGroup);

	for (const FXmlNode* VariantSet : FXmlNodeChildren(InVariantSetNode))
	{
		const FString& Tag = VariantSet->GetTag();
		const FString& SetRef = VariantSet->GetAttribute("ref");
		const FString& SetState = VariantSet->GetAttribute("state");

		if (Tag == TEXT("SequentialAnimation"))
		{
			OutVariantSet->VariantSet.bSequentialAnimation = VariantSet->GetContent() == TEXT("true");
			UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "VariantSet '%ls' has SequentialAnimation '%ls'", *OutVariantSet->Name, OutVariantSet->VariantSet.bSequentialAnimation ? TEXT("true") : TEXT("false"));
		}

		else if (Tag == TEXT("AnimationRef"))
		{
			OutVariantSet->VariantSet.AnimClips.Emplace(SetRef);
			UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "VariantSet '%ls' has AnimationRef '%ls'", *OutVariantSet->Name, *SetRef);
		}

		//Only actual variant sets have SetRef and SetState defined
		else if (!SetRef.IsEmpty() && !SetState.IsEmpty())
		{
			OutVariantSet->VariantSet.TargetVariantNames.Emplace(SetRef);
			OutVariantSet->VariantSet.ChosenOptions.Emplace(SetState);
		}
	}
}

// Returns a new string with the first character forced as upper case
FString FirstCharToUpper(const FString& InStr)
{
	if (InStr.Len() == 0)
	{
		return FString();
	}

	FString Result = InStr;
	Result[0] = Result.Left(1).ToUpper()[0];
	return Result;
}

bool LoadMatsFile(const TCHAR* InFilePath, FDatasmithVREDImportMatsResult& OutResult)
{
	FXmlFile MatsFile;

	TArray<FString> FileContentLines;
	if (!FFileHelper::LoadFileToStringArray(FileContentLines, InFilePath))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't read mats file: %ls", InFilePath);
	}

	FString FileContent = FString::Join(FileContentLines, TEXT("\n"));

	if (!MatsFile.LoadFile(FileContent, EConstructMethod::ConstructFromBuffer))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't open mats file: %ls", InFilePath);
		return false;
	}

	const FXmlNode* MatsNode = MatsFile.GetRootNode();
	if (!MatsNode)
	{
		return false;
	}

	static const TSet<FString> BoolParamNames({TEXT("useReflectivity"), TEXT("useOrangePeel")});

	static const TSet<FString> FloatParamNames({
		TEXT("roughness"), TEXT("displacementOffset"), TEXT("displacementHeight"),
		TEXT("reflectivity"), TEXT("bumpIntensity"), TEXT("refractionIndex"), TEXT("metalType"),
		TEXT("smear"), TEXT("contrast"), TEXT("saturation"), TEXT("roughnessU"), TEXT("roughnessV"),
		TEXT("clearcoatReflectivity"), TEXT("orangePeelFrequency"), TEXT("orangePeelIntensity")});

	static const TSet<FString> VectorParamNames({
		TEXT("diffuseColor"), TEXT("specularColor"), TEXT("interiorColor"), TEXT("exteriorColor"),
		TEXT("occlusionColor"), TEXT("scatterColor"), TEXT("incandescenceColor"), TEXT("reflectionColor"),
		TEXT("baseColor"), TEXT("flakeColor"), TEXT("clearcoatColor"), TEXT("triplanarRotate")});

	for (const FXmlNode* MatNode : FXmlNodeChildren(MatsNode))
	{
		if (!MatNode->GetTag().Equals(TEXT("Mat")))
		{
			continue;
		}

		FDatasmithFBXSceneMaterial* SceneMat = new(OutResult.Mats) FDatasmithFBXSceneMaterial;
		SceneMat->Name = MatNode->GetAttribute(TEXT("name"));
		SceneMat->Type = MatNode->GetAttribute(TEXT("matType"));

		for (const FXmlNode* ChildNode : FXmlNodeChildren(MatNode))
		{
			if (ChildNode->GetTag().Equals(TEXT("Users")))
			{
				// TODO - For now we'll just overwrite the fbx imported materials with this data, and those
				// are already correctly assigned
			}
			else if (ChildNode->GetTag().Equals(TEXT("Submaterials")))
			{
				// TODO - For now we'll just keep track of switch materials via the var file, as that is working
				// fine for now
			}
			else if (ChildNode->GetTag().Equals(TEXT("Properties")))
			{
				for (const FXmlNode* PropNode : FXmlNodeChildren(ChildNode))
				{
					const FString& PropName = PropNode->GetTag();

					if (VectorParamNames.Contains(PropName))
					{
						FVector4 Color;

						TArray<float> Floats;
						StringToFloatArray(PropNode->GetAttribute(TEXT("value")), Floats);
						for(int32 Index = 0; Index < Floats.Num() && Index < 4; Index++)
						{
							Color[Index] = Floats[Index];
						}

						// VRED does this
						if (PropName.Equals(TEXT("exteriorColor")) || PropName.Equals(TEXT("interiorColor")))
						{
							FLinearColor ColHSV = FLinearColor(Color).LinearRGBToHSV();
							Color.W = ColHSV.B;  // B corresponds to 'V'
						}

						SceneMat->VectorParams.Add(FirstCharToUpper(PropName), Color);
					}
					else if (FloatParamNames.Contains(PropName))
					{
						float Value = FCString::Atof(*PropNode->GetAttribute(TEXT("value")));

						SceneMat->ScalarParams.Add(FirstCharToUpper(PropName), Value);
					}
					else if (BoolParamNames.Contains(PropName))
					{
						static const FString BoolStr = TEXT("True");
						bool bValue = PropNode->GetAttribute(TEXT("value")).Equals(BoolStr);

						SceneMat->BoolParams.Add(FirstCharToUpper(PropName), bValue);
					}
					// Exceptions
					else if (PropName.Equals(TEXT("baseColor")))
					{
						// Some materials use "BaseColor" instead of DiffuseColor. Here we also
						// map them to DiffuseColor in case the materials are not supported (4.22).
						// Note that no VRED materials have both BaseColor and DiffuseColor,
						// so there is no chance for a harmful collision. We prevent an overwrite anyway

						FVector4 Color;

						TArray<float> Floats;
						StringToFloatArray(PropNode->GetAttribute(TEXT("value")), Floats);
						for(int32 Index = 0; Index < Floats.Num() && Index < 4; Index++)
						{
							Color[Index] = Floats[Index];
						}

						if (!SceneMat->VectorParams.Contains(TEXT("DiffuseColor")))
						{
							SceneMat->VectorParams.Add(TEXT("DiffuseColor"), Color);
						}
					}
					else if (PropName.Equals(TEXT("seeThrough")))
					{
						// VRED uses 1-opacity, or "seeThrough"
						// For some reason it stores opacity in a vector3, and averages the values
						TArray<float> Floats;
						StringToFloatArray(PropNode->GetAttribute(TEXT("value")), Floats);

						float Sum = 0.0f;
						for (float Val : Floats)
						{
							Sum += Val;
						}
						Sum /= Floats.Num();

						SceneMat->ScalarParams.Add(TEXT("Opacity"), 1.0f - Sum);
					}
					else if (PropName.Equals(TEXT("useFrostedGlass")) || PropName.Equals(TEXT("isRough")))
					{
						static const FString BoolStr = TEXT("True");
						bool bValue = PropNode->GetAttribute(TEXT("value")).Equals(BoolStr);

						SceneMat->BoolParams.Add(TEXT("UseRoughness"), bValue);
					}
					else if (PropName.Equals(TEXT("clearcoatType")))
					{
						int32 Value = FCString::Atoi(*PropNode->GetAttribute(TEXT("value")));

						SceneMat->BoolParams.Add(TEXT("UseClearCoat"), Value != 0);
					}
				}
			}
			else if (ChildNode->GetTag().Equals(TEXT("Textures")))
			{
				for (const FXmlNode* TexNode : FXmlNodeChildren(ChildNode))
				{
					if (TexNode->GetTag().Equals(TEXT("Tex")))
					{
						FString TexType = TexNode->GetAttribute(TEXT("texType"));
						FString Path = TexNode->GetAttribute(TEXT("exportedPath"));

						if (TexType.IsEmpty() || Path.IsEmpty())
						{
							UE_LOGF(LogDatasmithVREDImport, Warning, "Empty texture path ('%ls') or type ('%ls')", *Path, *TexType);
							continue;
						}

						TexType.RemoveFromEnd(TEXT("Component"), ESearchCase::CaseSensitive);
						TexType = TEXT("Tex") + FirstCharToUpper(TexType);

						FDatasmithFBXSceneMaterial::FTextureParams& Tex = SceneMat->TextureParams.FindOrAdd(TexType);
						Tex.Path = Path;

						for (const FXmlNode* TexChildNode : FXmlNodeChildren(TexNode))
						{
							if (TexChildNode->GetTag().Equals(TEXT("Properties")))
							{
								for (const FXmlNode* TexPropNode : FXmlNodeChildren(TexChildNode))
								{
									const FString& TexPropName = TexPropNode->GetTag();
									if (TexPropName.Equals(TEXT("planarTranslation")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.Translation = FVector4(Floats[0], Floats[1], Floats[2], 1.0f);
										}
									}
									if (TexPropName.Equals(TEXT("planarEulerRotation")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.Rotation = FVector4(Floats[0], Floats[1], Floats[2], 1.0f);
										}
									}
									if (TexPropName.Equals(TEXT("planarScale")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											// VRED emits planar scale z=1 when the user leaves it untouched (even if it shows zero in UI)
											// Untouched, the projection goes [-inf, -inf]. If the user manipulates it to
											// anything (even z=1, sadly), the projection goes from [0, -scale].
											// Sadly we lose information here on VRED's fault but we'll assume that
											// 1.0 really means "full range", as that is what it is by default in VRED
											// and it does seem to be the most common use case
											if (Floats[2] == 1.0f)
											{
												Floats[2] = FLT_MAX;
											}

											Tex.Scale = FVector4(Floats[0], Floats[1], Floats[2], 1.0f);
										}
									}
									else if (TexPropName.Equals(TEXT("color")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.Color = FVector4(Floats[0], Floats[1], Floats[2], 1.0f);
										}
									}
									else if (TexPropName.Equals(TEXT("offset")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 2)
										{
											Tex.Offset.X = Floats[0];
											Tex.Offset.Y = Floats[1];
										}
									}
									else if (TexPropName.Equals(TEXT("repeat")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 2)
										{
											Tex.Repeat.X = Floats[0];
											Tex.Repeat.Y = Floats[1];
										}
									}
									else if (TexPropName.Equals(TEXT("rotate")))
									{
										Tex.Rotate = FCString::Atof(*TexPropNode->GetAttribute(TEXT("value")));
									}
									else if (TexPropName.Equals(TEXT("textureProjectionType")))
									{
										Tex.ProjectionType = (ETextureMapType) FCString::Atoi(*TexPropNode->GetAttribute(TEXT("value")));
									}
									else if (TexPropName.Equals(TEXT("repeatMode")))
									{
										Tex.RepeatMode = (ETextureRepeatMode) FCString::Atoi(*TexPropNode->GetAttribute(TEXT("value")));
									}
									else if (TexPropName.Equals(TEXT("triplanarRotation")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.TriplanarRotation.X = Floats[0];
											Tex.TriplanarRotation.Y = Floats[1];
											Tex.TriplanarRotation.Z = Floats[2];
										}
									}
									else if (TexPropName.Equals(TEXT("triplanarOffsetU")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.TriplanarOffsetU.X = Floats[0];
											Tex.TriplanarOffsetU.Y = Floats[1];
											Tex.TriplanarOffsetU.Z = Floats[2];
										}
									}
									else if (TexPropName.Equals(TEXT("triplanarOffsetV")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.TriplanarOffsetV.X = Floats[0];
											Tex.TriplanarOffsetV.Y = Floats[1];
											Tex.TriplanarOffsetV.Z = Floats[2];
										}
									}
									else if (TexPropName.Equals(TEXT("triplanarRepeatU")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.TriplanarRepeatU.X = Floats[0];
											Tex.TriplanarRepeatU.Y = Floats[1];
											Tex.TriplanarRepeatU.Z = Floats[2];
										}
									}
									else if (TexPropName.Equals(TEXT("triplanarRepeatV")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 3)
										{
											Tex.TriplanarRepeatV.X = Floats[0];
											Tex.TriplanarRepeatV.Y = Floats[1];
											Tex.TriplanarRepeatV.Z = Floats[2];
										}
									}
									else if (TexPropName.Equals(TEXT("textureSizeMM")))
									{
										TArray<float> Floats;
										StringToFloatArray(TexPropNode->GetAttribute(TEXT("value")), Floats);
										if (Floats.Num() >= 2)
										{
											Tex.TriplanarTextureSize.X = Floats[0];
											Tex.TriplanarTextureSize.Y = Floats[1];
										}
									}
									else if (TexPropName.Equals(TEXT("triplanarBlend")))
									{
										float VREDBlend = FMath::Clamp(FCString::Atof(*TexPropNode->GetAttribute(TEXT("value"))), 0.0f, 1.0f);
										// Experimentally determined to match how it looks in VRED
										Tex.TriplanarBlendBias =  -0.54f + (VREDBlend/0.2f)*0.1f;
									}
								}
							}
						}
					}
				}
			}
		}
	}

	return true;
}

bool LoadVarFile(const TCHAR* InFilePath, FDatasmithVREDImportVariantsResult& OutResult)
{
	FXmlFile VarFile;

	TArray<FString> FileContentLines;
	if(!FFileHelper::LoadFileToStringArray(FileContentLines, InFilePath))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't read VAR file: %ls", InFilePath);
	}

	FixVREDXml(FileContentLines);

	// it would have been nice to have FXmlFile to expose LoadFile for lines(it splits into lines anyway inside)
	FString FileContent = FString::Join(FileContentLines, TEXT("\n"));

	if (!VarFile.LoadFile(FileContent, EConstructMethod::ConstructFromBuffer))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't open VAR file: %ls", *VarFile.GetLastError());
		return false;
	}

	const FXmlNode* VariantsNode = VarFile.GetRootNode();
	if (!VariantsNode)
	{
		return false;
	}

	// If we have no variants, VRED will still emit an XML file with just <Variants/>
	TArray<FXmlNode*> VariantTypeNodes = VariantsNode->GetChildrenNodes();
	if (VariantTypeNodes.Num() == 0)
	{
		return false;
	}

	for (const FXmlNode* VariantTypeNode : FXmlNodeChildren(VariantsNode))
	{
		// Parse variant set within a group
		if (VariantTypeNode->GetTag() == TEXT("VariantGroup"))
		{
			FString VariantGroupName = VariantTypeNode->GetAttribute(TEXT("name"));

			for (const FXmlNode* VariantSet : FXmlNodeChildren(VariantTypeNode))
			{
				if (VariantSet->GetTag() == TEXT("VariantSet"))
				{
					FVREDCppVariant* VariantSwitch = new(OutResult.VariantSwitches) FVREDCppVariant;
					ParseVariantSet(VariantSet, VariantSwitch, VariantGroupName);
				}
			}

			continue;
		}

		FVREDCppVariant* VariantSwitch = new(OutResult.VariantSwitches) FVREDCppVariant;

		VariantSwitch->Name = VariantTypeNode->GetAttribute("name");
		const FString& TargetNodeName = VariantTypeNode->GetAttribute("base");

		// Parse variant sets if they appear outside groups as well
		if (VariantTypeNode->GetTag() == TEXT("VariantSet"))
		{
			ParseVariantSet(VariantTypeNode, VariantSwitch, TEXT(""));
		}
		//Geometry variants
		else if (VariantTypeNode->GetTag() == TEXT("NodeVariant"))
		{
			UE_LOGF(LogDatasmithVREDImport, Verbose, "Geometry variant '%ls' imported", *TargetNodeName);
			VariantSwitch->Type = EVREDCppVariantType::Geometry;

			// Collect all command (!All, !Enable, etc) and non-command States of this switch
			TArray<FString> Commands;
			for (const FXmlNode* StateNode : FXmlNodeChildren(VariantTypeNode))
			{
				if (StateNode->GetTag() != TEXT("State"))
				{
					continue;
				}

				const FString& StateName = StateNode->GetAttribute("name");

				//Ignore states like !Next or !All
				if (StateName[0] == *TEXT("!"))
				{
					Commands.Emplace(StateName);
				}
				else
				{
					VariantSwitch->Geometry.TargetNodes.Emplace(StateName);
				}
			}

			//If we have explicit non-command states, we're a switch, so lets create some variant sets
			//It's important these are named after the visible mesh for package variants
			for (const FString& Geom : VariantSwitch->Geometry.TargetNodes)
			{
				UE_LOGF(LogDatasmithVREDImport, Verbose, "\tGeometry variant '%ls' has state %ls", *TargetNodeName, *Geom);

				FVREDCppVariantGeometryOption* MeshVariant = new(VariantSwitch->Geometry.Options) FVREDCppVariantGeometryOption;

				MeshVariant->Name = Geom;
				MeshVariant->VisibleMeshes.Emplace(Geom);

				for (const FString& other : VariantSwitch->Geometry.TargetNodes)
				{
					if (!Geom.Equals(other))
					{
						MeshVariant->HiddenMeshes.Emplace(other);
					}
				}
			}

			//Parse commands
			for (const FString& Command : Commands)
			{
				if (Command.Equals("!Enable"))
				{
					VariantSwitch->Geometry.TargetNodes.AddUnique(TargetNodeName);

					UE_LOGF(LogDatasmithVREDImport, Verbose, "\tGeometry variant '%ls' has state %ls", *TargetNodeName, *Command);
					FVREDCppVariantGeometryOption* MeshVariant = new(VariantSwitch->Geometry.Options) FVREDCppVariantGeometryOption;
					MeshVariant->Name = Command;
					MeshVariant->VisibleMeshes.Emplace(TargetNodeName);
					MeshVariant->HiddenMeshes.Empty();
				}

				else if (Command.Equals("!Disable"))
				{
					VariantSwitch->Geometry.TargetNodes.AddUnique(TargetNodeName);

					UE_LOGF(LogDatasmithVREDImport, Verbose, "\tGeometry variant '%ls' has state %ls", *TargetNodeName, *Command);
					FVREDCppVariantGeometryOption* MeshVariant = new(VariantSwitch->Geometry.Options) FVREDCppVariantGeometryOption;
					MeshVariant->Name = Command;
					MeshVariant->VisibleMeshes.Empty();
					MeshVariant->HiddenMeshes.Emplace(TargetNodeName);
				}

				else if (Command.Equals("!All"))
				{
					UE_LOGF(LogDatasmithVREDImport, Verbose, "\tGeometry variant '%ls' has state %ls", *TargetNodeName, *Command);
					FVREDCppVariantGeometryOption* MeshVariant = new(VariantSwitch->Geometry.Options) FVREDCppVariantGeometryOption;
					MeshVariant->Name = Command;
					MeshVariant->VisibleMeshes.Append(VariantSwitch->Geometry.TargetNodes);
					MeshVariant->HiddenMeshes.Empty();
				}

				else if (Command.Equals("!None"))
				{
					UE_LOGF(LogDatasmithVREDImport, Verbose, "\tGeometry variant '%ls' has state %ls", *TargetNodeName, *Command);
					FVREDCppVariantGeometryOption* MeshVariant = new(VariantSwitch->Geometry.Options) FVREDCppVariantGeometryOption;
					MeshVariant->Name = Command;
					MeshVariant->VisibleMeshes.Empty();
					MeshVariant->HiddenMeshes.Append(VariantSwitch->Geometry.TargetNodes);
				}
			}
		}
		//Light variants (behave almost exactly like geometry nodes, but switching them on/off shouldn't switch their entire tree)
		else if (VariantTypeNode->GetTag() == "LightVariant")
		{
			UE_LOGF(LogDatasmithVREDImport, Verbose, "Light variant '%ls' imported", *TargetNodeName);
			VariantSwitch->Type = EVREDCppVariantType::Light;

			VariantSwitch->Light.TargetNodes.Emplace(TargetNodeName);

			// All light variants just have two states: On/off
			FVREDCppVariantLightOption* LightVariant = nullptr;

			LightVariant = new(VariantSwitch->Light.Options) FVREDCppVariantLightOption;
			LightVariant->Name = "!Enable";

			LightVariant = new(VariantSwitch->Light.Options) FVREDCppVariantLightOption;
			LightVariant->Name = "!Disable";
		}
		//Material variants
		else if (VariantTypeNode->GetTag() == "MaterialVariant")
		{
			UE_LOGF(LogDatasmithVREDImport, Verbose, "Material variant '%ls' imported", *TargetNodeName);
			VariantSwitch->Type = EVREDCppVariantType::Material;

			for (const FXmlNode* Node : FXmlNodeChildren(VariantTypeNode))
			{
				// Points to a node using this switch material
				if (Node->GetTag() == "UsedBy")
				{
					const FString& StateName = Node->GetAttribute("name");

					UE_LOGF(LogDatasmithVREDImport, Verbose, "\tMaterial variant '%ls' is used by node %ls", *TargetNodeName, *StateName);
					VariantSwitch->Material.TargetNodes.Emplace(Node->GetAttribute("name"));
				}
				// State of the material switch, either a ! command or a submaterial
				else if (Node->GetTag() == "State")
				{
					const FString& StateName = Node->GetAttribute("name");

					// Ignore states like !Next or !All
					if (StateName[0] == *TEXT("!"))
					{
						continue;
					}

					// Found one of the submaterials. Just keep track of it by name
					UE_LOGF(LogDatasmithVREDImport, Verbose, "\tMaterial variant '%ls' has submaterial %ls", *TargetNodeName, *StateName);
					FVREDCppVariantMaterialOption* MatVariant = new(VariantSwitch->Material.Options) FVREDCppVariantMaterialOption;
					MatVariant->Name = StateName;
				}
			}
		}
		else if (VariantTypeNode->GetTag() == "OverlayVariant")
		{
			UE_LOGF(LogDatasmithVREDImport, Warning, "Overlay variant '%ls' unsupported", *TargetNodeName);
		}
		else if (VariantTypeNode->GetTag() == "TransformVariant")
		{
			UE_LOGF(LogDatasmithVREDImport, Verbose, "Transform variant '%ls' imported", *TargetNodeName);
			VariantSwitch->Type = EVREDCppVariantType::Transform;
			VariantSwitch->Transform.TargetNodes.Emplace(TargetNodeName);

			for (const FXmlNode* Node : FXmlNodeChildren(VariantTypeNode))
			{
				if (Node->GetTag() == "State")
				{
					const FString& StateName = Node->GetAttribute("name");

					// Ignore states like !Next or !Previous
					if (StateName[0] == *TEXT("!"))
					{
						continue;
					}

					// Found a transform variant
					UE_LOGF(LogDatasmithVREDImport, Verbose, "\tTransform variant '%ls' state %ls", *TargetNodeName, *StateName);
					FVREDCppVariantTransformOption* TransVariant = new(VariantSwitch->Transform.Options) FVREDCppVariantTransformOption;
					TransVariant->Name = StateName;

					const FString& TransformData = Node->GetAttribute("value");

					TArray<float> TransformFloats;
					StringToFloatArray(TransformData, TransformFloats);
					bool bSuccess = FloatArrayToTransform(TransformFloats, TransVariant->Transform);
					if ( !bSuccess )
					{
						UE_LOGF(LogDatasmithVREDImport, Warning, "Transform variant '%ls' state '%ls' has invalid transform value '%ls'. Transform will be set to identity.", *TargetNodeName, *StateName, *TransformData);
					}
				}
			}
		}
	}

	return true;
}

bool LoadLightsFile(const TCHAR* InFilePath, FDatasmithVREDImportLightsResult& OutResult)
{
	FXmlFile VarFile;

	TArray<FString> FileContentLines;
	if (!FFileHelper::LoadFileToStringArray(FileContentLines, InFilePath))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't read lights file: %ls", InFilePath);
	}

	FString FileContent = FString::Join(FileContentLines, TEXT("\n"));

	if (!VarFile.LoadFile(FileContent, EConstructMethod::ConstructFromBuffer))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't open lights file: %ls", InFilePath);
		return false;
	}

	const FXmlNode* LightsNode = VarFile.GetRootNode();
	if (!LightsNode)
	{
		return false;
	}

	for (const FXmlNode* Light : FXmlNodeChildren(LightsNode))
	{
		if (!Light->GetTag().Equals(TEXT("Light")))
		{
			continue;
		}

		FDatasmithFBXSceneLight* SceneLight = new(OutResult.Lights) FDatasmithFBXSceneLight;
		SceneLight->Name = Light->GetAttribute(TEXT("name"));

		for (const FXmlNode* AttribNode : FXmlNodeChildren(Light))
		{
			if (AttribNode->GetTag().Equals(TEXT("Type")))
			{
				FString lightType = AttribNode->GetAttribute(TEXT("value"));
				if (lightType.Equals(TEXT("point")))
				{
					SceneLight->LightType = ELightType::Point;
				}
				else if (lightType.Equals(TEXT("directional")))
				{
					SceneLight->LightType = ELightType::Directional;
				}
				else if (lightType.Equals(TEXT("spotlight")))
				{
					SceneLight->LightType = ELightType::Spot;
				}
				else if (lightType.Equals(TEXT("disk")))
				{
					SceneLight->LightType = ELightType::Area;
					SceneLight->AreaLightShape = EDatasmithLightShape::Disc;
				}
				else if (lightType.Equals(TEXT("rectangular")))
				{
					SceneLight->LightType = ELightType::Area;
					SceneLight->AreaLightShape = EDatasmithLightShape::Rectangle;
				}
				else if (lightType.Equals(TEXT("spherical")))
				{
					SceneLight->LightType = ELightType::Area;
					SceneLight->AreaLightShape = EDatasmithLightShape::Sphere;
				}
			}
			else if (AttribNode->GetTag().Equals(TEXT("Enabled")))
			{
				SceneLight->Enabled = AttribNode->GetAttribute(TEXT("value")).Equals(TEXT("True"));
			}
			else if (AttribNode->GetTag().Equals(TEXT("UseTemperature")))
			{
				SceneLight->UseTemperature = AttribNode->GetAttribute("value").Equals(TEXT("True"));
			}
			else if (AttribNode->GetTag().Equals(TEXT("Temperature")))
			{
				SceneLight->Temperature = FCString::Atod(*AttribNode->GetAttribute(TEXT("value")));
			}
			else if (AttribNode->GetTag().Equals(TEXT("Intensity")))
			{
				SceneLight->Intensity = FCString::Atod(*AttribNode->GetAttribute(TEXT("value")));
			}
			else if (AttribNode->GetTag().Equals(TEXT("DiffuseColor")))
			{
				TArray<float> Floats;
				StringToFloatArray(AttribNode->GetAttribute(TEXT("value")), Floats);

				if (Floats.Num() > 2)
				{
					SceneLight->DiffuseColor = FLinearColor(Floats[0], Floats[1], Floats[2]);
				}
			}
			else if (AttribNode->GetTag().Equals(TEXT("GlossyColor")))
			{
				TArray<float> Floats;
				StringToFloatArray(AttribNode->GetAttribute(TEXT("value")), Floats);

				if (Floats.Num() > 2)
				{
					SceneLight->UnusedGlossyColor = FLinearColor(Floats[0], Floats[1], Floats[2]);
				}
			}
			else if (AttribNode->GetTag().Equals(TEXT("ConeAngle")))
			{
				SceneLight->ConeOuterAngle = FCString::Atod(*AttribNode->GetAttribute(TEXT("value")));
			}
			else if (AttribNode->GetTag().Equals(TEXT("PenumbraAngle")))
			{
				SceneLight->ConeInnerAngle = FCString::Atod(*AttribNode->GetAttribute(TEXT("value")));
			}
			else if (AttribNode->GetTag().Equals(TEXT("AreaLightUseConeAngle")))
			{
				SceneLight->AreaLightUseConeAngle = AttribNode->GetAttribute(TEXT("value")).Equals(TEXT("True"));
			}
			else if (AttribNode->GetTag().Equals(TEXT("VisualizationVisible")))
			{
				SceneLight->VisualizationVisible = AttribNode->GetAttribute(TEXT("value")).Equals(TEXT("True"));
			}
			else if (AttribNode->GetTag().Equals(TEXT("AttenuationType")))
			{
				SceneLight->AttenuationType = (EAttenuationType)FCString::Atoi(*AttribNode->GetAttribute(TEXT("value")));
			}
			else if (AttribNode->GetTag().Equals(TEXT("Unit")))
			{
				SceneLight->Unit = FCString::Atoi(*AttribNode->GetAttribute(TEXT("value")));
			}
			else if (AttribNode->GetTag().Equals(TEXT("UseIESProfile")))
			{
				SceneLight->UseIESProfile = AttribNode->GetAttribute(TEXT("value")).Equals(TEXT("True"));
			}
			else if (AttribNode->GetTag().Equals(TEXT("IESPath")))
			{
				SceneLight->IESPath = AttribNode->GetAttribute(TEXT("value"));
			}
		}
	}

	return true;
}

bool LoadClipsFileForAnimNodes(const TCHAR* InFilePath, FDatasmithVREDImportClipsResult& OutResult)
{
	TArray<FString> FileContentLines;
	if (!FFileHelper::LoadFileToStringArray(FileContentLines, InFilePath))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't read .clips file: %ls", InFilePath);
	}

	FString FileContent = FString::Join(FileContentLines, TEXT("\n"));

	FXmlFile ClipsFile;
	if (!ClipsFile.LoadFile(FileContent, EConstructMethod::ConstructFromBuffer))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't open .clips file: %ls", *ClipsFile.GetLastError());
		return false;
	}

	const FXmlNode* RootNode = ClipsFile.GetRootNode();
	if (!RootNode)
	{
		UE_LOGF(LogDatasmithVREDImport, Warning, "Couldn't find a root node in .clips file: %ls", InFilePath);
		return false;
	}

	for (const FXmlNode* RootNodeChild : FXmlNodeChildren(RootNode))
	{
		if (RootNodeChild->GetTag() == TEXT("KeyTime"))
		{
			OutResult.KeyTime = FCString::Atof(*RootNodeChild->GetAttribute(TEXT("value")));
			UE_LOGF(LogDatasmithVREDImport, Verbose, "KeyTime = %f", OutResult.KeyTime);
		}
		else if (RootNodeChild->GetTag() == TEXT("BaseTime"))
		{
			OutResult.BaseTime = FCString::Atof(*RootNodeChild->GetAttribute(TEXT("value")));
			UE_LOGF(LogDatasmithVREDImport, Verbose, "BaseTime = %f", OutResult.BaseTime);
		}
		else if (RootNodeChild->GetTag() == TEXT("PlaybackSpeed"))
		{
			OutResult.PlaybackSpeed = FCString::Atof(*RootNodeChild->GetAttribute(TEXT("value")));
			UE_LOGF(LogDatasmithVREDImport, Verbose, "PlaybackSpeed = %f", OutResult.PlaybackSpeed);
		}
		else if (RootNodeChild->GetTag() == TEXT("Blocks"))
		{
			if (RootNodeChild->GetChildrenNodes().Num() == 0)
			{
				UE_LOGF(LogDatasmithVREDImport, Verbose, ".clips file %ls describes no animations", InFilePath);
				return false;
			}

			for (const FXmlNode* AnimNodeXML : FXmlNodeChildren(RootNodeChild))
			{
				FDatasmithFBXSceneAnimNode* AnimNode = new(OutResult.AnimNodes) FDatasmithFBXSceneAnimNode;
				AnimNode->Name = AnimNodeXML->GetAttribute(TEXT("name"));

				for (const FXmlNode* AnimBlockXML : FXmlNodeChildren(AnimNodeXML))
				{
					FDatasmithFBXSceneAnimBlock* AnimBlock = new(AnimNode->Blocks) FDatasmithFBXSceneAnimBlock;
					AnimBlock->Name = FDatasmithUtils::SanitizeObjectName(AnimBlockXML->GetAttribute(TEXT("name")));

					for (const FXmlNode* AnimCurveXML : FXmlNodeChildren(AnimBlockXML))
					{
						FDatasmithFBXSceneAnimCurve* AnimCurve = new(AnimBlock->Curves) FDatasmithFBXSceneAnimCurve;
						AnimCurve->DSID = FCString::Atoi(*AnimCurveXML->GetAttribute(TEXT("DSID")));
						AnimCurve->StartTimeSeconds = FCString::Atof(*AnimCurveXML->GetAttribute(TEXT("startSeconds")));

						FString CurveName = AnimCurveXML->GetAttribute(TEXT("name"));
						if (CurveName.StartsWith(TEXT("translation")))
						{
							AnimCurve->Type = EDatasmithFBXSceneAnimationCurveType::Translation;
						}
						else if (CurveName.StartsWith(TEXT("rotationOrientation")))
						{
							AnimCurve->Type = EDatasmithFBXSceneAnimationCurveType::Invalid;
						}
						else if (CurveName.StartsWith(TEXT("rotation")))
						{
							AnimCurve->Type = EDatasmithFBXSceneAnimationCurveType::Rotation;
						}
						else if (CurveName.StartsWith(TEXT("scale")))
						{
							AnimCurve->Type = EDatasmithFBXSceneAnimationCurveType::Scale;
						}
						else if (CurveName.StartsWith(TEXT("visible")))
						{
							AnimCurve->Type = EDatasmithFBXSceneAnimationCurveType::Visible;
						}

						if (CurveName.EndsWith(TEXT("X")))
						{
							AnimCurve->Component = EDatasmithFBXSceneAnimationCurveComponent::X;
						}
						else if (CurveName.EndsWith(TEXT("Y")))
						{
							AnimCurve->Component = EDatasmithFBXSceneAnimationCurveComponent::Y;
						}
						else if (CurveName.EndsWith(TEXT("Z")))
						{
							AnimCurve->Component = EDatasmithFBXSceneAnimationCurveComponent::Z;
						}

						UE_LOGF(LogDatasmithVREDImport, Verbose, "Parsed AnimNode %ls, AnimBlock %ls, AnimCurve with DSID %d and StartTimeSeconds %f", *AnimNode->Name, *AnimBlock->Name, AnimCurve->DSID, AnimCurve->StartTimeSeconds);
					}
				}
			}
		}
	}

	return true;
}

bool LoadClipsFileForClips(const TCHAR* InFilePath, FDatasmithVREDImportClipsResult& OutResult)
{
	TArray<FString> FileContentLines;
	if (!FFileHelper::LoadFileToStringArray(FileContentLines, InFilePath))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't read .clips file: %ls", InFilePath);
	}

	FString FileContent = FString::Join(FileContentLines, TEXT("\n"));

	FXmlFile ClipsFile;
	if (!ClipsFile.LoadFile(FileContent, EConstructMethod::ConstructFromBuffer))
	{
		UE_LOGF(LogDatasmithVREDImport, Error, "Couldn't open .clips file: %ls", *ClipsFile.GetLastError());
		return false;
	}

	const FXmlNode* RootNode = ClipsFile.GetRootNode();
	if (!RootNode)
	{
		UE_LOGF(LogDatasmithVREDImport, Warning, "Couldn't find a root node in .clips file: %ls", InFilePath);
		return false;
	}

	for (const FXmlNode* RootNodeChild : FXmlNodeChildren(RootNode))
	{
		if (RootNodeChild->GetTag() == TEXT("Clips"))
		{
			if (RootNodeChild->GetChildrenNodes().Num() == 0)
			{
				UE_LOGF(LogDatasmithVREDImport, Verbose, ".clips file %ls describes no clips", InFilePath);
				return false;
			}

			// Make sure OutResult.AnimClips doesn't reallocate when we add
			int32 NumClips = RootNodeChild->GetChildrenNodes().Num();
			OutResult.AnimClips.Reserve(NumClips);

			for (const FXmlNode* ClipXMLNode : FXmlNodeChildren(RootNodeChild))
			{
				FDatasmithFBXSceneAnimClip* ClipNode = new(OutResult.AnimClips) FDatasmithFBXSceneAnimClip;
				ClipNode->Name = FDatasmithUtils::SanitizeObjectName(ClipXMLNode->GetAttribute(TEXT("name")));
				ClipNode->bIsFlipped = ClipXMLNode->GetAttribute(TEXT("isFlipped")).Equals(TEXT("True"));

				for (const FXmlNode* BlockXMLNode : FXmlNodeChildren(ClipXMLNode))
				{
					FDatasmithFBXSceneAnimUsage* BlockUsage = new(ClipNode->AnimUsages) FDatasmithFBXSceneAnimUsage;
					BlockUsage->AnimName = FDatasmithUtils::SanitizeObjectName(BlockXMLNode->GetAttribute(TEXT("blockName")));
					BlockUsage->StartTime = FCString::Atof(*BlockXMLNode->GetAttribute(TEXT("startSeconds")));
					BlockUsage->EndTime = FCString::Atof(*BlockXMLNode->GetAttribute(TEXT("endSeconds")));
					BlockUsage->IsActive = BlockXMLNode->GetAttribute(TEXT("isActive")).Equals(TEXT("True"));
					BlockUsage->bIsFlipped = BlockXMLNode->GetAttribute(TEXT("isFlipped")).Equals(TEXT("True"));

					UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "Parsed block usage %ls, StartTime: %f, EndTime: %f, IsActive: %ls", *BlockUsage->AnimName, BlockUsage->StartTime, BlockUsage->EndTime, BlockUsage->IsActive ? TEXT("true") : TEXT("false"));
				}

				UE_LOGF(LogDatasmithVREDImport, VeryVerbose, "Parsed clip %ls, BlockUsage count: %d", *ClipNode->Name, ClipNode->AnimUsages.Num());
			}
		}
	}
	return true;
}

FDatasmithVREDImportMatsResult FDatasmithVREDAuxFiles::ParseMatsFile(const FString& InFilePath)
{
	FDatasmithVREDImportMatsResult Result;
	if (InFilePath.IsEmpty())
	{
		return Result;
	}

	if (FPaths::FileExists(InFilePath) && FPaths::GetExtension(InFilePath, false) == TEXT("mats"))
	{
		LoadMatsFile(*InFilePath, Result);
		UE_LOGF(LogDatasmithVREDImport, Verbose, "Imported %d materials from '%ls'", Result.Mats.Num(), *InFilePath);
	}
	else
	{
		UE_LOGF(LogDatasmithVREDImport, Warning, "Mats file '%ls' doesn't exist or is not a .mats file!", *InFilePath);
	}

	return Result;
}

FDatasmithVREDImportVariantsResult FDatasmithVREDAuxFiles::ParseVarFile(const FString& InFilePath)
{
	FDatasmithVREDImportVariantsResult Result;
	if (InFilePath.IsEmpty())
	{
		return Result;
	}

	if (FPaths::FileExists(InFilePath) && FPaths::GetExtension(InFilePath, false) == TEXT("var"))
	{
		LoadVarFile(*InFilePath, Result);

		for (const FVREDCppVariant& VariantSet : Result.VariantSwitches)
		{
			if (EVREDCppVariantType::Geometry == VariantSet.Type)
			{
				Result.SwitchObjects.Append(VariantSet.Geometry.TargetNodes);
			}
			if (EVREDCppVariantType::Material == VariantSet.Type)
			{
				Result.SwitchMaterialObjects.Append(VariantSet.Material.TargetNodes);
			}
			if (EVREDCppVariantType::Transform == VariantSet.Type)
			{
				Result.TransformVariantObjects.Append(VariantSet.Transform.TargetNodes);
			}
		}

		UE_LOGF(LogDatasmithVREDImport, Verbose, "Imported %d variants/variant sets", Result.VariantSwitches.Num());
	}
	else
	{
		UE_LOGF(LogDatasmithVREDImport, Warning, "Variants file '%ls' doesn't exist or is not a .var file!", *InFilePath);
	}

	return Result;
}

FDatasmithVREDImportLightsResult FDatasmithVREDAuxFiles::ParseLightsFile(const FString& InFilePath)
{
	FDatasmithVREDImportLightsResult Result;
	if (InFilePath.IsEmpty())
	{
		return Result;
	}

	if (FPaths::FileExists(InFilePath) && FPaths::GetExtension(InFilePath, false) == TEXT("lights"))
	{
		LoadLightsFile(*InFilePath, Result);
		UE_LOGF(LogDatasmithVREDImport, Verbose, "Imported extra information about %d lights", Result.Lights.Num());
	}
	else
	{
		UE_LOGF(LogDatasmithVREDImport, Warning, "Lights file '%ls' doesn't exist or is not a .lights file!", *InFilePath);
	}

	return Result;
}

FDatasmithVREDImportClipsResult FDatasmithVREDAuxFiles::ParseClipsFile(const FString& InFilePath)
{
	FDatasmithVREDImportClipsResult Result;
	if (InFilePath.IsEmpty())
	{
		return Result;
	}

	if (FPaths::FileExists(InFilePath) && FPaths::GetExtension(InFilePath, false) == TEXT("clips"))
	{
		if (LoadClipsFileForAnimNodes(*InFilePath, Result))
		{
			LoadClipsFileForClips(*InFilePath, Result);
		}
		UE_LOGF(LogDatasmithVREDImport, Verbose, "Imported %d AnimNodes and %d AnimClips", Result.AnimNodes.Num(), Result.AnimClips.Num());
	}
	else
	{
		UE_LOGF(LogDatasmithVREDImport, Warning, "Clips file '%ls' doesn't exist or is not a .clips file!", *InFilePath);
	}

	return Result;
}
