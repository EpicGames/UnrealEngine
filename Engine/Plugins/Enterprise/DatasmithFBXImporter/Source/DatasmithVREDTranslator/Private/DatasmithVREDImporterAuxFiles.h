// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include <cfloat>

struct FScriptContainerElement;

struct FDatasmithFBXSceneAnimClip;
struct FDatasmithFBXSceneAnimNode;
struct FDatasmithFBXSceneLight;
struct FDatasmithFBXSceneMaterial;
struct FVREDCppVariant;

class FDatasmithVREDImportMatsResult
{
public:
	TArray<FDatasmithFBXSceneMaterial> Mats;
};

class FDatasmithVREDImportVariantsResult
{
public:
	TArray<FVREDCppVariant> VariantSwitches;
	TSet<FString> SwitchObjects;
	TSet<FString> SwitchMaterialObjects;
	TSet<FString> TransformVariantObjects;
};

class FDatasmithVREDImportLightsResult
{
public:
	TArray<FDatasmithFBXSceneLight> Lights;
};

class FDatasmithVREDImportClipsResult
{
public:
	TArray<FDatasmithFBXSceneAnimNode> AnimNodes;
	TArray<FDatasmithFBXSceneAnimClip> AnimClips;
	float KeyTime = FLT_MAX;
	float BaseTime = 24.0f;
	float PlaybackSpeed = 24.0f;
};

namespace FDatasmithVREDAuxFiles
{
	FDatasmithVREDImportMatsResult ParseMatsFile(const FString& InFilePath);
	FDatasmithVREDImportVariantsResult ParseVarFile(const FString& InFilePath);
	FDatasmithVREDImportLightsResult ParseLightsFile(const FString& InFilePath);
	FDatasmithVREDImportClipsResult ParseClipsFile(const FString& InFilePath);
};
