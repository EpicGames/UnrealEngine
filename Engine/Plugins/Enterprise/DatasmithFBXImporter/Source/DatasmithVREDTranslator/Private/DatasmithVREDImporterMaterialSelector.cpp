// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDImporterMaterialSelector.h"

#include "IDatasmithSceneElements.h"

#include "Materials/MaterialInstanceConstant.h"

#define DEFAULT_MATERIAL_NAME TEXT("UPlasticMaterial")

FDatasmithVREDImporterMaterialSelector::FDatasmithVREDImporterMaterialSelector()
{
	FDatasmithReferenceMaterial& Phong = ReferenceMaterials.Add(TEXT("UPhongMaterial"));
	Phong.FromSoftObjectPath(FSoftObjectPath(TEXT("/DatasmithContent/Materials/FBXImporter/VRED/Phong.Phong")));

	FDatasmithReferenceMaterial& Plastic = ReferenceMaterials.Add(TEXT("UPlasticMaterial"));
	Plastic.FromSoftObjectPath(FSoftObjectPath(TEXT("/DatasmithContent/Materials/FBXImporter/VRED/Plastic.Plastic")));

	FDatasmithReferenceMaterial& Glass = ReferenceMaterials.Add(TEXT("UGlassMaterial"));
	Glass.FromSoftObjectPath(FSoftObjectPath(TEXT("/DatasmithContent/Materials/FBXImporter/VRED/Glass.Glass")));

	FDatasmithReferenceMaterial& Chrome = ReferenceMaterials.Add(TEXT("UChromeMaterial"));
	Chrome.FromSoftObjectPath(FSoftObjectPath(TEXT("/DatasmithContent/Materials/FBXImporter/VRED/Chrome.Chrome")));

	FDatasmithReferenceMaterial& BrushedMetal = ReferenceMaterials.Add(TEXT("UBrushedMetalMaterial"));
	BrushedMetal.FromSoftObjectPath(FSoftObjectPath(TEXT("/DatasmithContent/Materials/FBXImporter/VRED/BrushedMetal.BrushedMetal")));

	FDatasmithReferenceMaterial& UnicolorCarpaint = ReferenceMaterials.Add(TEXT("UUnicolorPaintMaterial"));
	UnicolorCarpaint.FromSoftObjectPath(FSoftObjectPath(TEXT("/DatasmithContent/Materials/FBXImporter/VRED/UnicolorCarpaint.UnicolorCarpaint")));
}

bool FDatasmithVREDImporterMaterialSelector::IsValid() const
{
	for (const auto& Pair : ReferenceMaterials)
	{
		const FDatasmithReferenceMaterial& Mat = Pair.Value;
		if (!Mat.IsValid())
		{
			return false;
		}
	}

	if (!ReferenceMaterials.Contains(DEFAULT_MATERIAL_NAME))
	{
		return false;
	}

	return true;
}

const FDatasmithReferenceMaterial& FDatasmithVREDImporterMaterialSelector::GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const
{
	const TSharedPtr< IDatasmithKeyValueProperty > TypeProperty = InDatasmithMaterial->GetPropertyByName(TEXT("Type"));
	FString TypeValue;
	if (TypeProperty.IsValid() && GetString(TypeProperty, TypeValue))
	{
		if (const FDatasmithReferenceMaterial* FoundMat = ReferenceMaterials.Find(TypeValue))
		{
			return *FoundMat;
		}
	}

	return ReferenceMaterials[DEFAULT_MATERIAL_NAME];
}

void FDatasmithVREDImporterMaterialSelector::FinalizeMaterialInstance(const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial, UMaterialInstanceConstant* MaterialInstance) const
{
	const TSharedPtr<IDatasmithKeyValueProperty> OpacityProperty = InDatasmithMaterial->GetPropertyByName(TEXT("Opacity"));
	const TSharedPtr<IDatasmithKeyValueProperty> TransparencyTextureProperty = InDatasmithMaterial->GetPropertyByName(TEXT("TexTransparencyIsActive"));
	const TSharedPtr< IDatasmithKeyValueProperty > TypeProperty = InDatasmithMaterial->GetPropertyByName(TEXT("Type"));

	FString TypeValue;
	if (TypeProperty.IsValid())
	{
		GetString(TypeProperty, TypeValue);
	}
	bool bIsGlassMaterial = TypeValue == TEXT("UGlassMaterial");

	float Opacity = 1.0f;
	bool bTransparencyActive = false;

	// If we have a non UGlassMaterial that has translucency or a translucency texture, we enable blend mode override to
	// make it render in translucent mode
	if (!bIsGlassMaterial &&
		((OpacityProperty.IsValid() && GetFloat(OpacityProperty, Opacity) && Opacity < 1.0f) ||
		(TransparencyTextureProperty.IsValid() && GetBool(TransparencyTextureProperty, bTransparencyActive) && bTransparencyActive)))
	{
		MaterialInstance->BasePropertyOverrides.bOverride_BlendMode = true;
		MaterialInstance->BasePropertyOverrides.BlendMode = EBlendMode::BLEND_Translucent;
	}
}

bool FDatasmithVREDImporterMaterialSelector::IsValidMaterialType( EDatasmithReferenceMaterialType InType ) const
{
	return InType == EDatasmithReferenceMaterialType::Auto || InType == EDatasmithReferenceMaterialType::Opaque || InType == EDatasmithReferenceMaterialType::Transparent;
}
