// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ReferenceMaterials/DatasmithReferenceMaterial.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialSelector.h"


enum class EDatasmithReferenceMaterialType : uint8;

class IDatasmithMaterialInstanceElement;

class FDatasmithVREDImporterMaterialSelector : public FDatasmithReferenceMaterialSelector
{
public:
	FDatasmithVREDImporterMaterialSelector();

	virtual bool IsValid() const override;
	virtual const FDatasmithReferenceMaterial& GetReferenceMaterial( const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial ) const override;
	virtual void FinalizeMaterialInstance(const TSharedPtr< IDatasmithMaterialInstanceElement >& InDatasmithMaterial, UMaterialInstanceConstant* MaterialInstance) const override;

protected:
	bool IsValidMaterialType( EDatasmithReferenceMaterialType InType ) const;

private:
	TMap<FString, FDatasmithReferenceMaterial> ReferenceMaterials;
};
