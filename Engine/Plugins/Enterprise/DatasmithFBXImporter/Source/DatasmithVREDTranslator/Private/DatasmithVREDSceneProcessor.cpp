// Copyright Epic Games, Inc. All Rights Reserved.


#include "DatasmithVREDSceneProcessor.h"
#include "DatasmithVREDLog.h"

FDatasmithVREDSceneProcessor::FDatasmithVREDSceneProcessor(FDatasmithFBXScene* InScene)
	: FDatasmithFBXSceneProcessor(InScene)
{
}

void FDatasmithVREDSceneProcessor::AddExtraLightInfo(TArray<FDatasmithFBXSceneLight>* InExtraLightsInfo)
{
	// Speed up lookups
	ExtraLightsInfo.Empty();
	for (FDatasmithFBXSceneLight l : *InExtraLightsInfo)
	{
		ExtraLightsInfo.Add(l.Name, MakeShared<FDatasmithFBXSceneLight>(l));
	}

	AddExtraLightNodesRecursive(Scene->RootNode);

	ExtraLightsInfo.Empty();
};

void FDatasmithVREDSceneProcessor::AddExtraLightNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node)
{
	TSharedPtr<FDatasmithFBXSceneLight>* ExtraInfo = ExtraLightsInfo.Find(Node->Name);
	if (ExtraInfo != nullptr)
	{
		// Take a deep copy from ExtraInfo
		Node->Light = TSharedPtr<FDatasmithFBXSceneLight>(new FDatasmithFBXSceneLight(**ExtraInfo));
		UE_LOGF(LogDatasmithVREDImport, Log, "Adding extra info to light '%ls'", *Node->Name);
	}

	for (TSharedPtr<FDatasmithFBXSceneNode> Child : Node->Children)
	{
		AddExtraLightNodesRecursive(Child);
	}
}

void FDatasmithVREDSceneProcessor::AddMatsMaterials(TArray<FDatasmithFBXSceneMaterial>* InMatsMaterials)
{
	TMap<FString, FDatasmithFBXSceneMaterial*> ExistingMats;
	for (TSharedPtr<FDatasmithFBXSceneMaterial>& ExistingMat : Scene->Materials)
	{
		ExistingMats.Add(ExistingMat->Name, ExistingMat.Get());
	}

	for (const FDatasmithFBXSceneMaterial& InMat : *InMatsMaterials)
	{
		FDatasmithFBXSceneMaterial** FoundMat = ExistingMats.Find(InMat.Name);
		if (FoundMat)
		{
			**FoundMat = InMat;
		}
		else
		{
			TSharedPtr<FDatasmithFBXSceneMaterial> AddedMat = MakeShared<FDatasmithFBXSceneMaterial>();
			*AddedMat = InMat;
			Scene->Materials.Add(AddedMat);
		}
	}
}

void FDatasmithVREDSceneProcessor::DecomposeRotationPivotsForNode(TSharedPtr<FDatasmithFBXSceneNode> Node, TMap<FString, FDatasmithFBXSceneAnimNode*>& NodeNameToAnimNode, TArray<FDatasmithFBXSceneAnimNode>& NewAnimNodes)
{
	if (!Node.IsValid() || Node->RotationPivot.IsNearlyZero())
	{
		return;
	}

	TSharedPtr<FDatasmithFBXSceneNode> NodeParent = Node->Parent.Pin();
	if (!NodeParent.IsValid())
	{
		return;
	}

	FVector RotPivot = Node->RotationPivot;
	FVector NodeLocation = Node->LocalTransform.GetTranslation();
	FQuat NodeRotation = Node->LocalTransform.GetRotation();

	Node->RotationPivot.Set(0.0f, 0.0f, 0.0f);
	Node->LocalTransform.SetTranslation(-RotPivot);
	Node->LocalTransform.SetRotation(FQuat::Identity);

	TSharedPtr<FDatasmithFBXSceneNode> Dummy = MakeShared<FDatasmithFBXSceneNode>();
	Dummy->Name = Node->Name + TEXT("_RotationPivot");
	Dummy->OriginalName = Dummy->Name;
	Dummy->SplitNodeID = Node->SplitNodeID;
	Dummy->LocalTransform.SetTranslation(NodeLocation + RotPivot);
	Dummy->LocalTransform.SetRotation(NodeRotation);

	// Move any rotation curves to Dummy
	if (FDatasmithFBXSceneAnimNode** FoundNode = NodeNameToAnimNode.Find(Node->OriginalName))
	{
		FDatasmithFBXSceneAnimNode* NewAnimNode = nullptr;

		for (FDatasmithFBXSceneAnimBlock& Block : (*FoundNode)->Blocks)
		{
			TArray<FDatasmithFBXSceneAnimCurve> RotCurves;
			TArray<FDatasmithFBXSceneAnimCurve> TransCurves;

			// Collect all rotation curves frm the original animnode
			for (int32 CurveIndex = Block.Curves.Num() - 1; CurveIndex >= 0; --CurveIndex)
			{
				FDatasmithFBXSceneAnimCurve& ThisCurve = Block.Curves[CurveIndex];

				if (ThisCurve.Type == EDatasmithFBXSceneAnimationCurveType::Rotation)
				{
					RotCurves.Add(ThisCurve);
					Block.Curves.RemoveAt(CurveIndex);
				}
				else if (ThisCurve.Type == EDatasmithFBXSceneAnimationCurveType::Translation)
				{
					for (FDatasmithFBXSceneAnimPoint& Pt : ThisCurve.Points)
					{
						Pt.Value += (RotPivot[(uint8)ThisCurve.Component] + Node->RotationOffset[(uint8)ThisCurve.Component] + Node->ScalingOffset[(uint8)ThisCurve.Component]);
					}
					TransCurves.Add(ThisCurve);
					Block.Curves.RemoveAt(CurveIndex);
				}
			}

			// Move curves to a new block on the new animnode
			if (RotCurves.Num() > 0 || TransCurves.Num() > 0)
			{
				if (NewAnimNode == nullptr)
				{
					NewAnimNode = new(NewAnimNodes) FDatasmithFBXSceneAnimNode;
					NewAnimNode->Name = Dummy->Name;
					Dummy->KeepNode();
				}

				FDatasmithFBXSceneAnimBlock* NewBlock = new(NewAnimNode->Blocks) FDatasmithFBXSceneAnimBlock;
				NewBlock->Name = Block.Name;
				NewBlock->Curves = MoveTemp(RotCurves);
				NewBlock->Curves.Append(TransCurves);
			}
		}
	}

	// Fix hierarchy (place Dummy between Node and Parent)
	Dummy->AddChild(Node);
	NodeParent->Children.Remove(Node);
	NodeParent->AddChild(Dummy);
}

void FDatasmithVREDSceneProcessor::DecomposeRotationPivots()
{
	TMap<FString, FDatasmithFBXSceneAnimNode*> NodeNameToAnimNode;
	for (FDatasmithFBXSceneAnimNode& AnimNode : Scene->AnimNodes)
	{
		NodeNameToAnimNode.Add(AnimNode.Name, &AnimNode);
	}

	TArray<FDatasmithFBXSceneAnimNode> NewNodes;

	for (TSharedPtr<FDatasmithFBXSceneNode> Node : Scene->GetAllNodes())
	{
		DecomposeRotationPivotsForNode(Node, NodeNameToAnimNode, NewNodes);
	}

	Scene->AnimNodes.Append(NewNodes);
}

void FDatasmithVREDSceneProcessor::DecomposeScalingPivotsForNode(TSharedPtr<FDatasmithFBXSceneNode> Node, TMap<FString, FDatasmithFBXSceneAnimNode*>& NodeNameToAnimNode, TArray<FDatasmithFBXSceneAnimNode>& NewAnimNodes)
{
	if (!Node.IsValid() || Node->ScalingPivot.IsNearlyZero())
	{
		return;
	}

	TSharedPtr<FDatasmithFBXSceneNode> NodeParent = Node->Parent.Pin();
	if (!NodeParent.IsValid())
	{
		return;
	}

	FVector ScalingPivot = Node->ScalingPivot;
	FVector NodeLocation = Node->LocalTransform.GetTranslation();
	FVector NodeScaling = Node->LocalTransform.GetScale3D();

	Node->ScalingPivot.Set(0.0f, 0.0f, 0.0f);
	Node->LocalTransform.SetTranslation(-ScalingPivot);
	Node->LocalTransform.SetScale3D(FVector::OneVector);

	TSharedPtr<FDatasmithFBXSceneNode> Dummy = MakeShared<FDatasmithFBXSceneNode>();
	Dummy->Name = Node->Name + TEXT("_ScalingPivot");
	Dummy->OriginalName = Dummy->Name;
	Dummy->SplitNodeID = Node->SplitNodeID;
	Dummy->LocalTransform.SetTranslation(NodeLocation + ScalingPivot);
	Dummy->LocalTransform.SetScale3D(NodeScaling);

	// Move any rotation curves to Dummy
	if (FDatasmithFBXSceneAnimNode** FoundNode = NodeNameToAnimNode.Find(Node->OriginalName))
	{
		FDatasmithFBXSceneAnimNode* NewAnimNode = nullptr;

		for (FDatasmithFBXSceneAnimBlock& Block : (*FoundNode)->Blocks)
		{
			TArray<FDatasmithFBXSceneAnimCurve> ScaleCurves;
			TArray<FDatasmithFBXSceneAnimCurve> TransCurves;

			// Collect all rotation curves frm the original animnode
			for (int32 CurveIndex = Block.Curves.Num() - 1; CurveIndex >= 0; --CurveIndex)
			{
				FDatasmithFBXSceneAnimCurve& ThisCurve = Block.Curves[CurveIndex];

				if (ThisCurve.Type == EDatasmithFBXSceneAnimationCurveType::Scale)
				{
					ScaleCurves.Add(ThisCurve);
					Block.Curves.RemoveAt(CurveIndex);
				}
				else if (ThisCurve.Type == EDatasmithFBXSceneAnimationCurveType::Translation)
				{
					for (FDatasmithFBXSceneAnimPoint& Pt : ThisCurve.Points)
					{
						Pt.Value += (ScalingPivot[(uint8)ThisCurve.Component] + Node->RotationOffset[(uint8)ThisCurve.Component] + Node->ScalingOffset[(uint8)ThisCurve.Component]);
					}
					TransCurves.Add(ThisCurve);
					Block.Curves.RemoveAt(CurveIndex);
				}
			}

			// Move curves to a new block on the new animnode
			if (ScaleCurves.Num() > 0 || TransCurves.Num() > 0)
			{
				if (NewAnimNode == nullptr)
				{
					NewAnimNode = new(NewAnimNodes) FDatasmithFBXSceneAnimNode;
					NewAnimNode->Name = Dummy->Name;
					Dummy->KeepNode();
				}

				FDatasmithFBXSceneAnimBlock* NewBlock = new(NewAnimNode->Blocks) FDatasmithFBXSceneAnimBlock;
				NewBlock->Name = Block.Name;
				NewBlock->Curves = MoveTemp(ScaleCurves);
				NewBlock->Curves.Append(TransCurves);
			}
		}
	}

	// Fix hierarchy (place Dummy between Node and Parent)
	Dummy->AddChild(Node);
	NodeParent->Children.Remove(Node);
	NodeParent->AddChild(Dummy);
}

void FDatasmithVREDSceneProcessor::DecomposeScalingPivots()
{
	TMap<FString, FDatasmithFBXSceneAnimNode*> NodeNameToAnimNode;
	for (FDatasmithFBXSceneAnimNode& AnimNode : Scene->AnimNodes)
	{
		NodeNameToAnimNode.Add(AnimNode.Name, &AnimNode);
	}

	TArray<FDatasmithFBXSceneAnimNode> NewNodes;

	for (TSharedPtr<FDatasmithFBXSceneNode> Node : Scene->GetAllNodes())
	{
		DecomposeScalingPivotsForNode(Node, NodeNameToAnimNode, NewNodes);
	}

	Scene->AnimNodes.Append(NewNodes);
}
