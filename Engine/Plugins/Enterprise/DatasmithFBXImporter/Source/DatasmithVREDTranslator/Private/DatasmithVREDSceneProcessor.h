// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithFBXSceneProcessor.h"

struct FDatasmithFBXScene;
struct FDatasmithFBXSceneLight;
struct FDatasmithFBXSceneMaterial;
struct FDatasmithFBXSceneNode;

class FDatasmithVREDSceneProcessor : public FDatasmithFBXSceneProcessor
{
public:
	FDatasmithVREDSceneProcessor(FDatasmithFBXScene* InScene);

	/** Add the extra info to the corresponding light nodes in the hierarchy */
	void AddExtraLightInfo(TArray<FDatasmithFBXSceneLight>* InExtraLightsInfo);

	/** Recursively add missing info to lights nodes */
	void AddExtraLightNodesRecursive(TSharedPtr<FDatasmithFBXSceneNode> Node);

	/** Overwrite FBX imported materials with mats material parameters and info */
	void AddMatsMaterials(TArray<FDatasmithFBXSceneMaterial>* InMatsMaterials);

	/** Decompose all scene nodes with nonzero RotationPivots using dummy actors, and handle their animations */
	void DecomposeRotationPivots();

	/** Recursively decompose nodes with nonzero RotationPivots using dummy actors, and handle their animations */
	void DecomposeRotationPivotsForNode(TSharedPtr<FDatasmithFBXSceneNode> Node, TMap<FString, FDatasmithFBXSceneAnimNode*>& NodeNameToAnimNode, TArray<FDatasmithFBXSceneAnimNode>& NewAnimNodes);

	/** Decompose all scene nodes with nonzero ScalingPivots using dummy actors, and handle their animations */
	void DecomposeScalingPivots();

	/** Recursively decompose nodes with nonzero ScalingPivots using dummy actors, and handle their animations */
	void DecomposeScalingPivotsForNode(TSharedPtr<FDatasmithFBXSceneNode> Node, TMap<FString, FDatasmithFBXSceneAnimNode*>& NodeNameToAnimNode, TArray<FDatasmithFBXSceneAnimNode>& NewAnimNodes);

protected:
	TMap<FString, TSharedPtr<FDatasmithFBXSceneLight>> ExtraLightsInfo;
};
