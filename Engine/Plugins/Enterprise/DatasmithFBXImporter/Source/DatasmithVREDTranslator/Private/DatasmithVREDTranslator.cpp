// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDTranslator.h"

#include "DatasmithPayload.h"
#include "DatasmithVREDImporter.h"
#include "DatasmithVREDImportOptions.h"
#include "DatasmithVREDLog.h"
#include "IDatasmithSceneElements.h"
#include "HAL/FileManager.h"

void FDatasmithVREDTranslator::Initialize(FDatasmithTranslatorCapabilities& OutCapabilities)
{
	OutCapabilities.bIsEnabled = true;
	OutCapabilities.bParallelLoadStaticMeshSupported = true;

	TArray<FFileFormatInfo>& Formats = OutCapabilities.SupportedFileFormats;
    Formats.Emplace(TEXT("fbx"), TEXT("VRED Fbx files"));
}

bool FDatasmithVREDTranslator::IsSourceSupported(const FDatasmithSceneSource& Source)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithVREDTranslator::IsSourceSupported)

	const FString& FilePath = Source.GetSourceFile();
	const FString& Extension = Source.GetSourceFileExtension();
	if (!Extension.Equals(TEXT("fbx"), ESearchCase::IgnoreCase))
	{
		return false;
	}

	FArchive* Reader = IFileManager::Get().CreateFileReader( *FilePath );
	if( !Reader )
	{
		return false;
	}

	ANSICHAR Header[64*1024] = { 0 };
	Reader->Serialize(Header, FMath::Min(Reader->TotalSize(), (int64)sizeof(Header) - 1));
	delete Reader;

	// Replace 0 with anything for Strstr to work on binary files
	for (int32 Index = 0; Index < sizeof(Header) - 1; ++Index)
	{
		if (Header[Index] == '\0')
		{
			Header[Index] = '.';
		}
	}

	// Quick and dirty way of identifying with a high degree of confidence if the file
	// is from VRED without parsing the whole scene using the SDK which can take a
	// long time for big scenes.
	//
	// Supports both ASCII and binary formats.
	bool bApplicationVendorValid = false;
	{
		const ANSICHAR* TagName = FPlatformString::Strstr(Header, "Original|ApplicationVendor");
		if (TagName)
		{
			const ANSICHAR* TagType = FPlatformString::Strstr(TagName, "KString");
			if (TagType)
			{
				const ANSICHAR* TagData = FPlatformString::Strstr(TagType, "Autodesk");
				if (TagData)
				{
					// The whole tag should be in the same vicinity
					bApplicationVendorValid = (TagData - TagName) < 256;
				}
			}
		}
	}
	if(!bApplicationVendorValid)
	{
		const ANSICHAR* TagName = FPlatformString::Strstr(Header, "LastSaved|ApplicationVendor");
		if (TagName)
		{
			const ANSICHAR* TagType = FPlatformString::Strstr(TagName, "KString");
			if (TagType)
			{
				const ANSICHAR* TagData = FPlatformString::Strstr(TagType, "Autodesk");
				if (TagData)
				{
					// The whole tag should be in the same vicinity
					bApplicationVendorValid = (TagData - TagName) < 256;
				}
			}
		}
	}

	bool bApplicationNameValid = false;
	{
		const ANSICHAR* TagName = FPlatformString::Strstr(Header, "Original|ApplicationName");
		if (TagName)
		{
			const ANSICHAR* TagType = FPlatformString::Strstr(TagName, "KString");
			if (TagType)
			{
				const ANSICHAR* TagData = FPlatformString::Strstr(TagType, "VRED");
				if (TagData)
				{
					// The whole tag should be in the same vicinity
					bApplicationNameValid = (TagData - TagName) < 256;
				}
			}
		}
	}
	if(!bApplicationNameValid)
	{
		const ANSICHAR* TagName = FPlatformString::Strstr(Header, "LastSaved|ApplicationName");
		if (TagName)
		{
			const ANSICHAR* TagType = FPlatformString::Strstr(TagName, "KString");
			if (TagType)
			{
				const ANSICHAR* TagData = FPlatformString::Strstr(TagType, "VRED");
				if (TagData)
				{
					// The whole tag should be in the same vicinity
					bApplicationNameValid = (TagData - TagName) < 256;
				}
			}
		}
	}

	return bApplicationVendorValid && bApplicationNameValid;
}

bool FDatasmithVREDTranslator::LoadScene(TSharedRef<IDatasmithScene> OutScene)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FDatasmithVREDTranslator::LoadScene)

	OutScene->SetHost(TEXT("VREDTranslator"));
	OutScene->SetProductName(TEXT("VRED"));

    Importer = MakeShared<FDatasmithVREDImporter>(OutScene, ImportOptions.Get());

	const FString& FilePath = GetSource().GetSourceFile();
	if(!Importer->OpenFile(FilePath))
	{
		UE_LOGF(LogDatasmithVREDImport, Log, "Failed to open file '%ls'!", *FilePath);
		return false;
	}

	if (!Importer->SendSceneToDatasmith())
	{
		UE_LOGF(LogDatasmithVREDImport, Log, "Failed to convert the VRED FBX scene '%ls' to Datasmith!", OutScene->GetName());
		return false;
	}

	return true;
}

void FDatasmithVREDTranslator::UnloadScene()
{
	if (Importer)
	{
		Importer->UnloadScene();
	}
}

bool FDatasmithVREDTranslator::LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload)
{
	if (ensure(Importer.IsValid()))
	{
		TArray<FMeshDescription> MeshDescriptions;
		Importer->GetGeometriesForMeshElementAndRelease(MeshElement, MeshDescriptions);
		if (MeshDescriptions.Num() > 0)
		{
			OutMeshPayload.LodMeshes.Add(MoveTemp(MeshDescriptions[0]));
			return true;
		}
	}

	return false;
}

bool FDatasmithVREDTranslator::LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload)
{
	//if (ensure(Importer.IsValid()))
	//{
	//	const TArray<TSharedRef<IDatasmithLevelSequenceElement>>& ImportedSequences = Importer->GetImportedSequences();
	//	if (ImportedSequences.Contains(LevelSequenceElement))
	//	{
	//		// #ueent_todo: move data to OutLevelSequencePayload
	//		// Right now the LevelSequenceElement is imported out of the IDatasmithScene
	//		return true;
	//	}
	//}

	return false;
}

void FDatasmithVREDTranslator::GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	if (!ImportOptions.IsValid())
	{
		const FString& FilePath = GetSource().GetSourceFile();

		ImportOptions = Datasmith::MakeOptions<UDatasmithVREDImportOptions>();
		ImportOptions->ResetPaths(FilePath, false);
	}

	Options.Add(ImportOptions.Get());
}

void FDatasmithVREDTranslator::SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options)
{
	for (const TObjectPtr<UDatasmithOptionsBase>& OptionPtr : Options)
	{
		if (UDatasmithVREDImportOptions* InImportOptions = Cast<UDatasmithVREDImportOptions>(OptionPtr))
		{
			ImportOptions.Reset(InImportOptions);
		}
	}

	if (Importer.IsValid())
	{
		Importer->SetImportOptions(ImportOptions.Get());
	}
}
