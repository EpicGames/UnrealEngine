// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DatasmithTranslator.h"

class FDatasmithSceneSource;
class UDatasmithVREDImportOptions;

class FDatasmithVREDTranslator : public IDatasmithTranslator
{
public:
	virtual FName GetFName() const override { return "DatasmithVREDTranslator"; };

	virtual void Initialize(FDatasmithTranslatorCapabilities& OutCapabilities) override;
	virtual bool IsSourceSupported(const FDatasmithSceneSource& Source) override;

	virtual bool LoadScene(TSharedRef<IDatasmithScene> OutScene) override;
	virtual void UnloadScene() override;

	virtual bool LoadStaticMesh(const TSharedRef<IDatasmithMeshElement> MeshElement, FDatasmithMeshElementPayload& OutMeshPayload) override;
	virtual bool LoadLevelSequence(const TSharedRef<IDatasmithLevelSequenceElement> LevelSequenceElement, FDatasmithLevelSequencePayload& OutLevelSequencePayload) override;

	virtual void GetSceneImportOptions(TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;
	virtual void SetSceneImportOptions(const TArray<TObjectPtr<UDatasmithOptionsBase>>& Options) override;

private:
    TStrongObjectPtr<UDatasmithVREDImportOptions> ImportOptions;
    TSharedPtr<class FDatasmithVREDImporter> Importer;
};
