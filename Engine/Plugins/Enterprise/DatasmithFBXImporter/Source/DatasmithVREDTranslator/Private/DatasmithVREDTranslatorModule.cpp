// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDTranslatorModule.h"

#include "DatasmithVREDImporterMaterialSelector.h"
#include "DatasmithVREDImportOptions.h" // IWYU pragma: keep
#include "DatasmithVREDTranslator.h"
#include "ReferenceMaterials/DatasmithReferenceMaterialManager.h"


class FVREDTranslatorModule : public IDatasmithVREDTranslatorModule
{
public:
	virtual void StartupModule() override
	{
		// Make sure the DatasmithImporter module exists and has been initialized before adding FDatasmithVREDTranslator's material selector
		FModuleManager::Get().LoadModule(TEXT("DatasmithImporter"));

		FDatasmithReferenceMaterialManager::Get().RegisterSelector(TEXT("VRED"), MakeShared< FDatasmithVREDImporterMaterialSelector >());

		Datasmith::RegisterTranslator<FDatasmithVREDTranslator>();
	}

	virtual void ShutdownModule() override
	{
		Datasmith::UnregisterTranslator<FDatasmithVREDTranslator>();
	}
};

IMPLEMENT_MODULE(FVREDTranslatorModule, DatasmithVREDTranslator);
