// Copyright Epic Games, Inc. All Rights Reserved.

#include "DatasmithVREDVariantConverter.h"

#include "DatasmithSceneFactory.h"
#include "DatasmithVariantElements.h"
#include "DatasmithVREDImportData.h"
#include "DatasmithVREDLog.h"

#define UNGROUPED_VARSET_NAME TEXT("Ungrouped variant sets")

namespace FVREDVariantConverterImpl
{
	// Returns the first child actor element of 'Parent' that is in the same split group (tag[1]) and
	// has EDatasmithElementType 'Type'. This is not recursive, as nodes are always split at one level
	// Note: This might return Parent itself
	TSharedPtr<IDatasmithActorElement> GetChildOfType(TSharedPtr<IDatasmithActorElement> Parent, EDatasmithElementType Type)
	{
		if (!Parent.IsValid())
		{
			return nullptr;
		}


		if (Parent->GetTagsCount() < 2)
		{
			return nullptr;
		}

		FString SplitGroup = Parent->GetTag(1);

		for (int32 ChildIndex = 0; ChildIndex < Parent->GetChildrenCount(); ++ChildIndex)
		{
			TSharedPtr<IDatasmithActorElement> Child = Parent->GetChild(ChildIndex);
			if (Child->GetTagsCount() >= 2 && Child->GetTag(1) == SplitGroup && Child->IsA(Type))
			{
				return Child;
			}
		}

		if (Parent->IsA(Type))
		{
			return Parent;
		}

		return nullptr;
	}

	// Adds data from InVariantA into InOutVariantB, combining bindings for the same actor
	// Note that this might lead to some property values/bindings being pointed to by multiple parents, but that
	// is fine: When creating the LevelVariantSets asset it will lead to creating separate variants like usual
	void MergeVariants(TSharedPtr<IDatasmithVariantElement> FromVariantA, TSharedPtr<IDatasmithVariantElement> IntoVariantB)
	{
		TMap<TWeakPtr<IDatasmithActorElement>, TSharedPtr<IDatasmithActorBindingElement>> BindingsInB;
		for (int32 BindingIndex = 0; BindingIndex < IntoVariantB->GetActorBindingsCount(); ++BindingIndex)
		{
			TSharedPtr<IDatasmithActorBindingElement> Binding = IntoVariantB->GetActorBinding(BindingIndex);
			TWeakPtr<IDatasmithActorElement> Actor = Binding->GetActor();
			BindingsInB.Add(Actor, Binding);
		}

		for (int32 BindingIndex = 0; BindingIndex < FromVariantA->GetActorBindingsCount(); ++BindingIndex)
		{
			TSharedPtr<IDatasmithActorBindingElement> BindingFromA = FromVariantA->GetActorBinding(BindingIndex);
			TWeakPtr<IDatasmithActorElement> Actor = BindingFromA->GetActor();

			// There is already a binding for Actor in IntoVariantB --> Move all property captures over
			// It doesn't matter if there are multiple captures to the same property, as only one UPropertyCapture is ever created
			if (TSharedPtr<IDatasmithActorBindingElement>* FoundBindingInB = BindingsInB.Find(Actor))
			{
				for (int32 PropIndex = 0; PropIndex < BindingFromA->GetPropertyCapturesCount(); ++PropIndex)
				{
					TSharedPtr<IDatasmithBasePropertyCaptureElement> PropertyCapture = BindingFromA->GetPropertyCapture(PropIndex);
					(*FoundBindingInB)->AddPropertyCapture(PropertyCapture.ToSharedRef());
				}
			}
			// There is no binding for Actor in IntoVariantB --> Move the entire binding to IntoVariantB
			else
			{
				IntoVariantB->AddActorBinding(BindingFromA.ToSharedRef());
				BindingsInB.Add(Actor, BindingFromA);
			}
		}
	}

	// Sanitize before giving it to Datasmith as it will otherwise end up
	// with an 'Object_' prefix
	FString SanitizeName(FString Name)
	{
		Name.RemoveFromStart(TEXT("!"));
		if (Name == TEXT("None"))
		{
			return TEXT("Nothing");
		}
		return Name;
	}
}

TSharedPtr<IDatasmithLevelVariantSetsElement> FVREDVariantConverter::ConvertVariants(TArray<FVREDCppVariant>& Vars, FActorMap& ActorsByOriginalName, FMaterialMap& MaterialsByName)
{
	if (Vars.Num() == 0)
	{
		return nullptr;
	}

	TSharedPtr<IDatasmithLevelVariantSetsElement> LVS = FDatasmithSceneFactory::CreateLevelVariantSets(TEXT("LevelVariantSets"));

	// Remember all vars and varsets we created so that we can sanitize their names
	// later and remove duplicates
	TMap<TSharedPtr<IDatasmithVariantElement>, FString> CreatedVariantsToOriginalNames;
	TMap<TSharedPtr<IDatasmithVariantSetElement>, FString> CreatedVariantSetsToOriginalNames;

	TMap<FString, FVREDCppVariant*> VREDVarsByName;
	TMap<FString, TArray<FVREDCppVariant*>> VREDVarSetsByGroup;
	for (FVREDCppVariant& Var : Vars)
	{
		if (Var.Type == EVREDCppVariantType::VariantSet)
		{
			FString GroupName = Var.VariantSet.VariantSetGroupName;
			if (GroupName.IsEmpty())
			{
				GroupName = UNGROUPED_VARSET_NAME;
			}

			TArray<FVREDCppVariant*>& Group = VREDVarSetsByGroup.FindOrAdd(GroupName);
			Group.Add(&Var);
		}
		else
		{
			VREDVarsByName.Add(Var.Name, &Var);
		}
	}

	// Create a VM variant set for each VRED variant
	for (auto Pair : VREDVarsByName)
	{
		FString VarName = Pair.Key;
		FVREDCppVariant* Var = Pair.Value;

		TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(*FVREDVariantConverterImpl::SanitizeName(VarName));
		LVS->AddVariantSet(NewVarSet.ToSharedRef());
		CreatedVariantSetsToOriginalNames.Add(NewVarSet, VarName);

		switch (Var->Type)
		{
		case EVREDCppVariantType::Camera:
			for (const FVREDCppVariantCameraOption& Option : Var->Camera.Options)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*FVREDVariantConverterImpl::SanitizeName(Option.Name));
				NewVarSet->AddVariant(NewVar.ToSharedRef());
				CreatedVariantsToOriginalNames.Add(NewVar, Option.Name);
			}
			break;
		case EVREDCppVariantType::Geometry:
		{
			TSet<TSharedPtr<IDatasmithActorElement>> NodeActors;
			for (const FString& NodeName : Var->Geometry.TargetNodes)
			{
				// For geometry variants, we don't need to check the children.
				// We always want the parent-most node with the tag so that we can hide
				// the entire hierarchy, which is what Actors.Find will return
				TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(FName(*NodeName));
				if (FoundActors)
				{
					NodeActors.Append(*FoundActors);
				}
				else
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "Did not find actor '%ls' used by variant '%ls'", *NodeName, *VarName);
				}
			}
			TArray<TSharedPtr<IDatasmithActorElement>> NodeActorsArray = NodeActors.Array();

			for (const FVREDCppVariantGeometryOption& Option : Var->Geometry.Options)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*FVREDVariantConverterImpl::SanitizeName(Option.Name));
				NewVarSet->AddVariant(NewVar.ToSharedRef());
				CreatedVariantsToOriginalNames.Add(NewVar, Option.Name);

				TSet<FString> VisibileNodes = TSet<FString>(Option.VisibleMeshes);

				for (const TSharedPtr<IDatasmithActorElement>& NodeActor : NodeActorsArray)
				{
					TSharedPtr<IDatasmithPropertyCaptureElement> PropertyCapture = FDatasmithSceneFactory::CreatePropertyCapture();
					PropertyCapture->SetCategory(EDatasmithPropertyCategory::Visibility);

					FString OriginalName = NodeActor->GetTagsCount() > 0 ? NodeActor->GetTag(0) : FString();
					bool bVisible = VisibileNodes.Contains(OriginalName);
					PropertyCapture->SetRecordedData((uint8*)&bVisible, sizeof(bool));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(NodeActor);
					Binding->AddPropertyCapture(PropertyCapture.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
			break;
		}
		case EVREDCppVariantType::Light:
		{
			TSet<TSharedPtr<IDatasmithActorElement>> NodeActors;
			for (const FString& NodeName : Var->Light.TargetNodes)
			{
				// For lights, we split the actors to match the correct rotation, so the
				// correct actor will be a child with the same tag and a light component
				if (TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(FName(*NodeName)))
				{
					for (TSharedPtr<IDatasmithActorElement> FoundActor : *FoundActors)
					{
						if (TSharedPtr<IDatasmithActorElement> TargetActor = FVREDVariantConverterImpl::GetChildOfType(FoundActor,
							EDatasmithElementType::Light |
							EDatasmithElementType::PointLight |
							EDatasmithElementType::DirectionalLight |
							EDatasmithElementType::AreaLight |
							EDatasmithElementType::LightmassPortal |
							EDatasmithElementType::EnvironmentLight))
						{
							NodeActors.Add(TargetActor);
						}
					}
				}
				else
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "Did not find actor '%ls' used by variant '%ls'", *NodeName, *VarName);
				}
			}
			TArray<TSharedPtr<IDatasmithActorElement>> NodeActorsArray = NodeActors.Array();

			for (const FVREDCppVariantLightOption& Option : Var->Light.Options)
			{
				// For light variants, we always just get two options: !Enable and !Disable, which toggle visibility
				// of the target node
				bool bVisible = Option.Name == TEXT("!Enable");

				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*FVREDVariantConverterImpl::SanitizeName(Option.Name));
				NewVarSet->AddVariant(NewVar.ToSharedRef());
				CreatedVariantsToOriginalNames.Add(NewVar, Option.Name);

				for (const TSharedPtr<IDatasmithActorElement>& NodeActor : NodeActorsArray)
				{
					TSharedPtr<IDatasmithPropertyCaptureElement> PropertyCapture = FDatasmithSceneFactory::CreatePropertyCapture();
					PropertyCapture->SetCategory(EDatasmithPropertyCategory::Visibility);
					PropertyCapture->SetRecordedData((uint8*)&bVisible, sizeof(bool));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(NodeActor);
					Binding->AddPropertyCapture(PropertyCapture.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
			break;
		}
		case EVREDCppVariantType::Material:
		{
			TSet<TSharedPtr<IDatasmithActorElement>> NodeActors;
			for (const FString& NodeName : Var->Material.TargetNodes)
			{
				if (TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(FName(*NodeName)))
				{
					for (TSharedPtr<IDatasmithActorElement> FoundActor : *FoundActors)
					{
						if (TSharedPtr<IDatasmithActorElement> TargetActor = FVREDVariantConverterImpl::GetChildOfType(FoundActor, EDatasmithElementType::StaticMeshActor))
						{
							NodeActors.Add(TargetActor);
						}
					}
				}
				else
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "Did not find actor '%ls' used by variant '%ls'", *NodeName, *VarName);
				}
			}
			TArray<TSharedPtr<IDatasmithActorElement>> NodeActorsArray = NodeActors.Array();

			for (const FVREDCppVariantMaterialOption& Option : Var->Material.Options)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*FVREDVariantConverterImpl::SanitizeName(Option.Name));
				NewVarSet->AddVariant(NewVar.ToSharedRef());
				CreatedVariantsToOriginalNames.Add(NewVar, Option.Name);

				const FString& MatName = Option.Name;
				TSharedPtr<IDatasmithBaseMaterialElement>* DatasmithMaterial = MaterialsByName.Find(FName(*MatName));
				if (!DatasmithMaterial || !DatasmithMaterial->IsValid())
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "Failed to find material '%ls' for variant set '%ls'", *MatName, NewVarSet->GetName());
					continue;
				}

				for (const TSharedPtr<IDatasmithActorElement>& NodeActor : NodeActorsArray)
				{
					TSharedPtr<IDatasmithObjectPropertyCaptureElement> PropertyCapture = FDatasmithSceneFactory::CreateObjectPropertyCapture();
					PropertyCapture->SetCategory(EDatasmithPropertyCategory::Material);
					PropertyCapture->SetRecordedObject(*DatasmithMaterial);

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(NodeActor);
					Binding->AddPropertyCapture(PropertyCapture.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
			break;
		}
		case EVREDCppVariantType::Transform:
		{
			TSet<TSharedPtr<IDatasmithActorElement>> NodeActors;
			for (const FString& NodeName : Var->Transform.TargetNodes)
			{
				if (TArray<TSharedPtr<IDatasmithActorElement>>* FoundActors = ActorsByOriginalName.Find(FName(*NodeName)))
				{
					NodeActors.Append(*FoundActors);
				}
				else
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "Did not find actor '%ls' used by variant '%ls'", *NodeName, *VarName);
				}
			}
			TArray<TSharedPtr<IDatasmithActorElement>> NodeActorsArray = NodeActors.Array();

			for (const FVREDCppVariantTransformOption& Option : Var->Transform.Options)
			{
				TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*FVREDVariantConverterImpl::SanitizeName(Option.Name));
				NewVarSet->AddVariant(NewVar.ToSharedRef());
				CreatedVariantsToOriginalNames.Add(NewVar, Option.Name);

				const FTransform& Trans = Option.Transform;
				FVector Loc = Trans.GetLocation();
				FRotator Rot = Trans.GetRotation().Rotator();
				FVector Scale = Trans.GetScale3D();

				for (const TSharedPtr<IDatasmithActorElement>& NodeActor : NodeActorsArray)
				{
					TSharedPtr<IDatasmithPropertyCaptureElement> LocationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					LocationProperty->SetCategory(EDatasmithPropertyCategory::RelativeLocation);
					LocationProperty->SetRecordedData((uint8*)&Loc, sizeof(FVector));

					TSharedPtr<IDatasmithPropertyCaptureElement> RotationProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					RotationProperty->SetCategory(EDatasmithPropertyCategory::RelativeRotation);
					RotationProperty->SetRecordedData((uint8*)&Rot, sizeof(FRotator));

					TSharedPtr<IDatasmithPropertyCaptureElement> ScaleProperty = FDatasmithSceneFactory::CreatePropertyCapture();
					ScaleProperty->SetCategory(EDatasmithPropertyCategory::RelativeScale3D);
					ScaleProperty->SetRecordedData((uint8*)&Scale, sizeof(FVector));

					TSharedPtr<IDatasmithActorBindingElement> Binding = FDatasmithSceneFactory::CreateActorBinding();
					Binding->SetActor(NodeActor);
					Binding->AddPropertyCapture(LocationProperty.ToSharedRef());
					Binding->AddPropertyCapture(RotationProperty.ToSharedRef());
					Binding->AddPropertyCapture(ScaleProperty.ToSharedRef());

					NewVar->AddActorBinding(Binding.ToSharedRef());
				}
			}
			break;
		}
		default:
			UE_LOGF(LogDatasmithVREDImport, Warning, "Unsupported variant type for variant '%ls'", *VarName);
			continue;
		}
	}

	// Convert each VRED variant set group into a variant set, basically flattening the properties
	// captured in the chosen option of each target VRED variant into the same "bucket" of a VM variant
	// Note that ungrouped VRED variant sets were already grouped into an 'Ungrouped variant sets' group
	for (TPair<FString, TArray<FVREDCppVariant*>>& Pair : VREDVarSetsByGroup)
	{
		FString GroupName = Pair.Key;
		TArray<FVREDCppVariant*>& VarSets = Pair.Value;

		// One Variant Manager Variant Set per VRED 'variant set group'
		TSharedPtr<IDatasmithVariantSetElement> NewVarSet = FDatasmithSceneFactory::CreateVariantSet(*FVREDVariantConverterImpl::SanitizeName(GroupName));
		LVS->AddVariantSet(NewVarSet.ToSharedRef());
		CreatedVariantSetsToOriginalNames.Add(NewVarSet, GroupName);

		for (const FVREDCppVariant* VarSet : VarSets)
		{
			// One Variant Manager Variant per VRED 'variant set'
			TSharedPtr<IDatasmithVariantElement> NewVar = FDatasmithSceneFactory::CreateVariant(*FVREDVariantConverterImpl::SanitizeName(VarSet->Name));
			NewVarSet->AddVariant(NewVar.ToSharedRef());
			CreatedVariantsToOriginalNames.Add(NewVar, VarSet->Name);

			const TArray<FString>& VMVariantSetNames = VarSet->VariantSet.TargetVariantNames;
			const TArray<FString>& VMVariantNames = VarSet->VariantSet.ChosenOptions;
			if (VMVariantSetNames.Num() != VMVariantNames.Num())
			{
				UE_LOGF(LogDatasmithVREDImport, Warning, "Number of target VRED variants differs from the number of chosen VRED variant options!");
				continue;
			}

			for (int32 Index = 0; Index < VMVariantNames.Num(); ++Index)
			{
				const FString& TargetVarSetName = VMVariantSetNames[Index];

				TSharedPtr<IDatasmithVariantSetElement> VMVariantSet = nullptr;
				for (const TPair<TSharedPtr<IDatasmithVariantSetElement>, FString>& VarSetPair : CreatedVariantSetsToOriginalNames)
				{
					if (TargetVarSetName == VarSetPair.Value)
					{
						VMVariantSet = VarSetPair.Key;
						break;
					}
				}
				if (!VMVariantSet.IsValid())
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "When trying to create a variant set for group '%ls', did not find created variant set '%ls'", *GroupName, *VMVariantSetNames[Index]);
					continue;
				}

				const FString& TargetVarName = VMVariantNames[Index];

				TSharedPtr<IDatasmithVariantElement> VMVariant = nullptr;
				for (int32 VarIndex = 0; VarIndex < VMVariantSet->GetVariantsCount(); ++VarIndex)
				{
					TSharedPtr<IDatasmithVariantElement> Var = VMVariantSet->GetVariant(VarIndex);
					FString* VarOriginalName = CreatedVariantsToOriginalNames.Find(Var);

					if (Var.IsValid() && VarOriginalName && *VarOriginalName == TargetVarName)
					{
						VMVariant = Var;
						break;
					}
				}
				if (!VMVariant.IsValid())
				{
					UE_LOGF(LogDatasmithVREDImport, Warning, "When trying to create a variant set for group '%ls', did not find created variant '%ls'", *GroupName, *VMVariantSetNames[Index]);
					continue;
				}

				// Append VMVariant into NewVar
				FVREDVariantConverterImpl::MergeVariants(VMVariant, NewVar);
			}
		}
	}

	return LVS;
}
