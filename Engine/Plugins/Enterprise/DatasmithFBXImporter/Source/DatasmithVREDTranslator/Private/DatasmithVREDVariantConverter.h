// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class AActor;
class IDatasmithActorElement;
class IDatasmithBaseMaterialElement;
class IDatasmithLevelVariantSetsElement;
struct FAssetData;
struct FVREDCppVariant;

using FActorMap = TMap<FName, TArray<TSharedPtr<IDatasmithActorElement>>>;
using FMaterialMap = TMap<FName, TSharedPtr<IDatasmithBaseMaterialElement>>;

class FVREDVariantConverter
{
public:
	static TSharedPtr<IDatasmithLevelVariantSetsElement> ConvertVariants(TArray<FVREDCppVariant>& Vars, FActorMap& ActorsByOriginalName, FMaterialMap& MaterialsByName);
};

