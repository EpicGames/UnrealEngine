// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#define VREDTRANSLATOR_MODULE_NAME TEXT("DatasmithVREDTranslator")

class IDatasmithVREDTranslatorModule : public IModuleInterface
{
public:
	static inline IDatasmithVREDTranslatorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IDatasmithVREDTranslatorModule >(VREDTRANSLATOR_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(VREDTRANSLATOR_MODULE_NAME);
	}
};
